// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"

class IPropertyHandle;
class STextBlock;
class URuntimeVirtualTexture;
class URuntimeVirtualTextureComponent;

/** UI customization for URuntimeVirtualTexture */
class FRuntimeVirtualTextureDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FRuntimeVirtualTextureDetailsCustomization();

	/** Refresh array of supported material types. */
	void RefreshMaterialTypes();

	/** Callback for updating text values after an edit. */
	void RefreshTextDetails();
	/** Callback for full update of details view after an edit. */
	void RefreshDetailsView();

	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	TWeakObjectPtr<URuntimeVirtualTexture> VirtualTexture;

	TArray<int32> SupportedMaterialTypes;

	IDetailLayoutBuilder* CachedDetailBuilder = nullptr;

	TSharedPtr<STextBlock> TileCountText;
	TSharedPtr<STextBlock> TileSizeText;
	TSharedPtr<STextBlock> TileBorderSizeText;

	TSharedPtr<STextBlock> SizeText;
	TSharedPtr<STextBlock> PageTableSizeText;
};


/** UI customization for URuntimeVirtualTextureComponent */
class FRuntimeVirtualTextureComponentDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FRuntimeVirtualTextureComponentDetailsCustomization();

	/** Returns true if SetBounds button is enabled */
	bool IsSetBoundsEnabled() const;
	/** Callback for Set Bounds button */
	FReply SetBounds();

	/** Returns true if BuildStreamedMips button is enabled */
	bool IsBuildStreamedMipsEnabled() const;
	/** Returns the visibility of the BuildAllStreamedMips button */
	EVisibility GetBuildAllStreamedMipsVisible() const;
	/** Returns true if the streaming texture is setup */
	bool IsStreamingTextureSet() const;
	/** Returns true if the streaming texture is up to date */
	bool IsStreamingTextureValid(FText& OutReason) const;
	/** Callback for Build button */
	FReply BuildStreamedMips(bool bBuildAll);

	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	TWeakObjectPtr<URuntimeVirtualTextureComponent> RuntimeVirtualTextureComponent;
};
