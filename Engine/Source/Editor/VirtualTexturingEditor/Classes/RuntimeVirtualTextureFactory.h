// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "RuntimeVirtualTextureFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

/** Factory for URuntimeVirtualTexture */
UCLASS(hidecategories = (Object))
class URuntimeVirtualTextureFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
