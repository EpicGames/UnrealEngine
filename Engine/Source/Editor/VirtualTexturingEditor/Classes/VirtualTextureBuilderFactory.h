// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "VirtualTextureBuilderFactory.generated.h"

/** Factory for UVirtualTextureBuilder */
UCLASS(hidecategories = (Object))
class UVirtualTextureBuilderFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ Begin UFactory Interface	
};
