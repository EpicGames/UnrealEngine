// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "MaterialCache/MaterialCacheVirtualTextureTag.h"
#include "AssetDefinition_MaterialCacheVirtualTextureTag.generated.h"

UCLASS()
class UAssetDefinition_MaterialCacheVirtualTextureTag : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_MaterialCacheVirtualTextureTag", "Material Cache Virtual Texture Tag"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialCacheVirtualTextureTag::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor(255, 100, 100); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "AssetDefinition_MaterialCacheVirtualTextureTagSubMenu", "Virtual Texture"), ECategoryMenuType::Section)
		};
		return Categories;
	}
	// UAssetDefinition End
};
