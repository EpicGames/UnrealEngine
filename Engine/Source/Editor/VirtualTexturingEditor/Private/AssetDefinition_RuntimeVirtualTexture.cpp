// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_RuntimeVirtualTexture.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "EditorSupportDelegates.h"
#include "FileHelpers.h"
#include "IContentBrowserSingleton.h"
#include "MaterialEditingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_RuntimeVirtualTexture"

DEFINE_LOG_CATEGORY_STATIC(LogRuntimeVirtualTextureFixMaterial, Log, Log);

namespace MenuExtension::RuntimeVirtualTexture
{
	template <class T> 
	void GetReferencersOfType(UObject *Object, TArray<T*> &OutObjects)
	{
		TArray<FAssetData> AssetDatas;
		UAssetRegistryHelpers::FindReferencersOfAssetOfClass(Object, { T::StaticClass() }, AssetDatas);

		for (auto Data : AssetDatas)
		{
			T* TypedAsset = Cast<T>(Data.GetAsset());
			if (TypedAsset != nullptr)
			{
				OutObjects.Add(TypedAsset);
			}
		}
	}

	void FindAllMaterials(URuntimeVirtualTexture* RuntimeVirtualTexture, TArray<UMaterial*>& OutMaterials, TArray<UMaterialFunctionInterface*>& OutFunctions)
	{
		TArray<UMaterialInterface*> MaterialInterfaces;
		GetReferencersOfType(RuntimeVirtualTexture, MaterialInterfaces);

		for (UMaterialInterface *MaterialInterface : MaterialInterfaces)
		{
			UMaterial* Material = Cast<UMaterial>(MaterialInterface);
			
			UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
			if (MaterialInstance != nullptr)
			{
				Material = MaterialInstance->GetMaterial();
			}

			OutMaterials.AddUnique(Material);
		}

		GetReferencersOfType(RuntimeVirtualTexture, OutFunctions);
	}

	void FixMaterialUsage(URuntimeVirtualTexture* RuntimeVirtualTexture)
	{
		TArray<UPackage*> PackagesToSave;
		{
			UE_LOG(LogRuntimeVirtualTextureFixMaterial, Log, TEXT("Begin fix material usage for '%s' ..."), *RuntimeVirtualTexture->GetName());

			TArray<UMaterial*> Materials;
			TArray<UMaterialFunctionInterface*> Functions;
			FindAllMaterials(RuntimeVirtualTexture, Materials, Functions);

			int32 TaskCount = Materials.Num() + Functions.Num();
			FScopedSlowTask Task(static_cast<float>(TaskCount), LOCTEXT("RuntimeVirtualTexture_FixMaterialUsageProgress", "Fixing materials for Runtime Virtual Texture usage..."));
			Task.MakeDialog();

			for (UMaterial* Material : Materials)
			{
				Task.EnterProgressFrame();

				bool bMaterialModified = false;
				for (UMaterialExpression* Expression : Material->GetExpressions())
				{
					UMaterialExpressionRuntimeVirtualTextureSample* RVTSampleExpression = Cast<UMaterialExpressionRuntimeVirtualTextureSample>(Expression);
					if (RVTSampleExpression)
					{
						if (RuntimeVirtualTexture == RVTSampleExpression->VirtualTexture)
						{
							if (RVTSampleExpression->InitVirtualTextureDependentSettings())
							{
								Expression->Modify();

								FPropertyChangedEvent Event(UMaterialExpressionTextureBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialExpressionRuntimeVirtualTextureSample, MaterialType)));
								Expression->PostEditChangeProperty(Event);

								bMaterialModified = true;
							}
						}
					}
				}

				if (bMaterialModified)
				{
					UE_LOG(LogRuntimeVirtualTextureFixMaterial, Log, TEXT("  Recompile material '%s' ..."), *Material->GetName());

					FScopedSlowTask CompileTask(1, FText::AsCultureInvariant(Material->GetName()));
					CompileTask.MakeDialog();
					CompileTask.EnterProgressFrame();

					UMaterialEditingLibrary::RecompileMaterial(Material);

					PackagesToSave.Add(Material->GetOutermost());
				}
			}

			for (UMaterialFunctionInterface *Function : Functions)
			{
				Task.EnterProgressFrame();

				bool bFunctionModified = false;
				for (const TObjectPtr<UMaterialExpression>& Expression : Function->GetExpressions())
				{
					UMaterialExpressionRuntimeVirtualTextureSample* RVTSampleExpression = Cast<UMaterialExpressionRuntimeVirtualTextureSample>(Expression);
					if (RVTSampleExpression)
					{
						if (RuntimeVirtualTexture == RVTSampleExpression->VirtualTexture)
						{
							if (RVTSampleExpression->InitVirtualTextureDependentSettings())
							{
								Expression->Modify();

								FPropertyChangedEvent Event(UMaterialExpressionTextureBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialExpressionRuntimeVirtualTextureSample, MaterialType)));
								Expression->PostEditChangeProperty(Event);

								bFunctionModified = true;
							}
						}
					}
				}

				if (bFunctionModified)
				{
					UE_LOG(LogRuntimeVirtualTextureFixMaterial, Log, TEXT("  Update function '%s' ..."), *Function->GetName());

					FScopedSlowTask CompileTask(1, FText::AsCultureInvariant(Function->GetName()));
					CompileTask.MakeDialog();
					CompileTask.EnterProgressFrame();

					UMaterialEditingLibrary::UpdateMaterialFunction(Function, nullptr);

					PackagesToSave.Add(Function->GetOutermost());
				}
			}

			UE_LOG(LogRuntimeVirtualTextureFixMaterial, Log, TEXT("End fix material usage for '%s' ..."), *RuntimeVirtualTexture->GetName());
		}
	
		if (PackagesToSave.Num())
		{
			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, true);
		}
	}

	void ExecuteFindMaterials(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		TArray<FAssetData> AssetData = CBContext->SelectedAssets;
		if (AssetData.Num() != 1)
		{
			return;
		}

		TArray<FAssetData> Materials;
		URuntimeVirtualTexture* RuntimeVirtualTexture = CBContext->LoadFirstSelectedObject<URuntimeVirtualTexture>();

		if (RuntimeVirtualTexture != nullptr)
		{
			UAssetRegistryHelpers::FindReferencersOfAssetOfClass(RuntimeVirtualTexture, { UMaterialInterface::StaticClass(), UMaterialFunction::StaticClass() }, Materials);
		}

		if (Materials.Num() > 0)
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(Materials);
		}
	}

	void ExecuteFixMaterialUsage(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		TArray<FAssetData> AssetData = CBContext->SelectedAssets;
		if (AssetData.Num() != 1)
		{
			return;
		}

		URuntimeVirtualTexture* RuntimeVirtualTexture = CBContext->LoadFirstSelectedObject<URuntimeVirtualTexture>();
		if (RuntimeVirtualTexture != nullptr)
		{
			FixMaterialUsage(RuntimeVirtualTexture);

			FEditorDelegates::RefreshEditor.Broadcast();
			FEditorSupportDelegates::RedrawAllViewports.Broadcast();
		}
	}

	bool IsOneAssetSelected(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return false;
		}

		TArray<FAssetData> AssetData = CBContext->SelectedAssets;
		return AssetData.Num() == 1;
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			
			// Runtime Virtual Texture Action Registration
			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(URuntimeVirtualTexture::StaticClass());

				FToolMenuSection& RuntimeVirtualTextureSection = Menu->FindOrAddSection("GetAssetActions");
				RuntimeVirtualTextureSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
						{
							if (Context->SelectedAssets.Num() > 0)
							{
								const FAssetData& AssetData = Context->SelectedAssets[0];
								if (AssetData.AssetClassPath == URuntimeVirtualTexture::StaticClass()->GetClassPathName())
								{
									const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(URuntimeVirtualTexture::StaticClass());
									
									// Find Material
									{
										const TAttribute<FText> Label = LOCTEXT("RuntimeVirtualTexture_FindMaterials", "Find Materials Using This");
										const TAttribute<FText> ToolTip = LOCTEXT("RuntimeVirtualTexture_FindMaterialsTooltip", "Finds all materials that use this texture in the content browser.");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindMaterials);
										UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsOneAssetSelected);
										InSection.AddMenuEntry("DialogWave_PlaySound", Label, ToolTip, Icon, UIAction);
									}

									// Fix Material
									{
										const TAttribute<FText> Label = LOCTEXT("RuntimeVirtualTexture_FixMaterialUsage", "Fix Material Usage");
										const TAttribute<FText> ToolTip = LOCTEXT("RuntimeVirtualTexture_FixMaterialUsageTooltip", "Find materials using this Runtime Virtual Texture and fix any mismatching content types.");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFixMaterialUsage);
										UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsOneAssetSelected);
										InSection.AddMenuEntry("DialogWave_StopSound", Label, ToolTip, Icon, UIAction);
									}
								}
							}
						}
					}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
