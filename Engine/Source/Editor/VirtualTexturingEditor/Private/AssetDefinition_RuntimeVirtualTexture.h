// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "VT/RuntimeVirtualTexture.h"
#include "AssetDefinition_RuntimeVirtualTexture.generated.h"

UCLASS()
class UAssetDefinition_RuntimeVirtualTexture : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_RuntimeVirtualTexture", "Runtime Virtual Texture"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(128, 128, 128); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return URuntimeVirtualTexture::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "AssetDefinition_RuntimeVirtualTextureSubMenu", "Virtual Texture"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
