// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "VT/VirtualTextureBuilder.h"
#include "AssetDefinition_VirtualTextureBuilder.generated.h"

UCLASS()
class UAssetDefinition_VirtualTextureBuilder : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_VirtualTextureBuilder", "Streaming Runtime Virtual Texture"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UVirtualTextureBuilder::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor(128, 128, 128); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "AssetDefinition_VirtualTextureBuilderSubMenu", "Virtual Texture"), ECategoryMenuType::Section)
		};
		return Categories;
	}
	// UAssetDefinition End
};
