// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCacheVirtualTextureTagFactory.h"
#include "MaterialCache/MaterialCacheVirtualTextureTag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialCacheVirtualTextureTagFactory)

UMaterialCacheVirtualTextureTagFactory::UMaterialCacheVirtualTextureTagFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialCacheVirtualTextureTag::StaticClass();
	
	bCreateNew    = true;
	bEditAfterNew = true;
	bEditorImport = false;
}

UObject* UMaterialCacheVirtualTextureTagFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMaterialCacheVirtualTextureTag* Tag = NewObject<UMaterialCacheVirtualTextureTag>(InParent, Class, Name, Flags);
	check(Tag);
	return Tag;
}
