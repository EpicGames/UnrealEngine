// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintVirtualTextureThumbnailRenderer.h"
#include "VT/MeshPaintVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintVirtualTextureThumbnailRenderer)

UMeshPaintVirtualTextureThumbnailRenderer::UMeshPaintVirtualTextureThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UMeshPaintVirtualTextureThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return UTextureThumbnailRenderer::CanVisualizeAsset(Object);
}

void UMeshPaintVirtualTextureThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UTextureThumbnailRenderer::Draw(Object, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
}
