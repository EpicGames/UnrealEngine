// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeVirtualTextureBuildStreamingMips.h"

#include "AssetCompilingManager.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ContentStreaming.h"
#include "EngineModule.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RendererInterface.h"
#include "RenderTargetPool.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "ShaderCompiler.h"
#include "UObject/Package.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureRender.h"
#include "VT/VirtualTextureBuilder.h"

#define LOCTEXT_NAMESPACE "VirtualTexturingEditorModule"

namespace
{
	/** Container for render resources needed to render the runtime virtual texture. */
	class FTileRenderResources : public FRenderResource
	{
	public:
		FTileRenderResources(int32 InTileSize, int32 InNumTilesX, int32 InNumTilesY, int32 InNumLayers, TArrayView<EPixelFormat> const& InLayerFormats)
			: TileSize(InTileSize)
			, NumLayers(InNumLayers)
			, TotalSizeBytes(0)
		{
			LayerFormats.SetNumZeroed(InNumLayers);
			LayerOffsets.SetNumZeroed(InNumLayers);

			for (int32 Layer = 0; Layer < NumLayers; ++Layer)
			{
				check(InLayerFormats[Layer] == PF_G16 || InLayerFormats[Layer] == PF_B8G8R8A8 || InLayerFormats[Layer] == PF_DXT1 || InLayerFormats[Layer] == PF_DXT5 || InLayerFormats[Layer] == PF_BC4
					|| InLayerFormats[Layer] == PF_BC5 || InLayerFormats[Layer] == PF_R5G6B5_UNORM || InLayerFormats[Layer] == PF_B5G5R5A1_UNORM);
				LayerFormats[Layer] = InLayerFormats[Layer] == PF_G16 || InLayerFormats[Layer] == PF_BC4 ? PF_G16 : PF_B8G8R8A8;
				LayerOffsets[Layer] = TotalSizeBytes;
				TotalSizeBytes += CalculateImageBytes(InTileSize, InTileSize, 0, LayerFormats[Layer]) * InNumTilesX * InNumTilesY;
			}
		}

		//~ Begin FRenderResource Interface.
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			RenderTargets.Init(nullptr, NumLayers);
			StagingTextures.Init(nullptr, NumLayers);

			for (int32 Layer = 0; Layer < NumLayers; ++Layer)
			{
				FPooledRenderTargetDesc RenderTargetDesc = FPooledRenderTargetDesc::Create2DDesc(
					FIntPoint(TileSize, TileSize), LayerFormats[Layer], FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource, false);
				GRenderTargetPool.FindFreeElement(RHICmdList, RenderTargetDesc, RenderTargets[Layer], TEXT("FTileRenderResources"));

				FRHITextureCreateDesc StagingTextureDesc = FRHITextureCreateDesc::Create2D(TEXT("FTileRenderResources"), TileSize, TileSize, LayerFormats[Layer]);
				StagingTextureDesc.SetFlags(ETextureCreateFlags::CPUReadback);
				StagingTextures[Layer] = RHICmdList.CreateTexture(StagingTextureDesc);
			}

			Fence = RHICreateGPUFence(TEXT("Runtime Virtual Texture Build"));
		}

		virtual void ReleaseRHI() override
		{
			RenderTargets.Empty();
			StagingTextures.Empty();
			Fence.SafeRelease();
		}
		//~ End FRenderResource Interface.

		int32 GetNumLayers() const { return NumLayers; }
		int64 GetTotalSizeBytes() const { return TotalSizeBytes; }

		EPixelFormat GetLayerFormat(int32 Index) const { return LayerFormats[Index]; }
		int64 GetLayerOffset(int32 Index) const { return LayerOffsets[Index]; }

		TRefCountPtr<IPooledRenderTarget> GetRenderTarget(int32 Index) const { return RenderTargets[Index]; }
		FRHITexture* GetStagingTexture(int32 Index) const { return StagingTextures[Index]; }
		FRHIGPUFence* GetFence() const { return Fence; }

	private:
		int32 TileSize;
		int32 NumLayers;
		int64 TotalSizeBytes;

		TArray<EPixelFormat> LayerFormats;
		TArray<int64> LayerOffsets;

		TArray<TRefCountPtr<IPooledRenderTarget>> RenderTargets;
		TArray<FTextureRHIRef> StagingTextures;
		FGPUFenceRHIRef Fence;
	};

	/** Templatized helper function for copying a rendered tile to the final composited image data. */
	template<typename T>
	void TCopyTile(T* SrcPixels, int32 TileSize, int32 SrcStride, T* DestPixels, int32 DestStride, int32 DestLayerStride, FIntPoint const& DestPos)
	{
		for (int32 y = 0; y < TileSize; y++)
		{
			memcpy(
				DestPixels + (SIZE_T)DestStride * ((SIZE_T)DestPos[1] + (SIZE_T)y) + DestPos[0],
				SrcPixels + SrcStride * y,
				TileSize * sizeof(T));
		}
	}

	/** Function for copying a rendered tile to the final composited image data. Needs ERuntimeVirtualTextureMaterialType to know what type of data is being copied. */
	void CopyTile(void* SrcPixels, int32 TileSize, int32 SrcStride, void* DestPixels, int32 DestStride, int32 DestLayerStride, FIntPoint const& DestPos, EPixelFormat Format)
	{
		check(Format == PF_G16 || Format == PF_B8G8R8A8);
		if (Format == PF_G16)
		{
			TCopyTile((uint16*)SrcPixels, TileSize, SrcStride, (uint16*)DestPixels, DestStride, DestLayerStride, DestPos);
		}
		else if (Format == PF_B8G8R8A8)
		{
			TCopyTile((FColor*)SrcPixels, TileSize, SrcStride, (FColor*)DestPixels, DestStride, DestLayerStride, DestPos);
		}
	}
}


namespace RuntimeVirtualTexture
{
	bool HasStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent)
	{
		if (InComponent == nullptr)
		{
			return false;
		}

		if (InComponent->GetVirtualTexture() == nullptr || InComponent->GetStreamingTexture() == nullptr)
		{
			return false;
		}

		if (InComponent->NumStreamingMips() <= 0)
		{
			return false;
		}

		if (ShadingPath == EShadingPath::Mobile && !InComponent->GetStreamingTexture()->bSeparateTextureForMobile)
		{
			return false;
		}

		return true;
	}

	bool BuildStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildStreamedMips);

		if (!HasStreamedMips(ShadingPath, InComponent))
		{
			return true;
		}

		if (!InComponent->IsRegistered())
		{
			UE_LOGF(LogVirtualTexturing, Error, "Trying to build streaming mips for a component (from actor %ls) that is not registered. It will be ignored", *InComponent->GetOwner()->GetActorNameOrLabel());
			return false;
		}

		URuntimeVirtualTexture* RuntimeVirtualTexture = InComponent->GetVirtualTexture();
		const int32 RuntimeVirtualTextureId = RuntimeVirtualTexture->GetUniqueID();
		FSceneInterface* Scene = InComponent->GetScene();
		checkf(Scene != nullptr, TEXT("Trying to build streaming mips for a component (from actor %s) that doesn't have a valid scene"), *InComponent->GetOwner()->GetActorNameOrLabel());

		const FTransform Transform = InComponent->GetComponentTransform();
		const FBox Bounds = InComponent->Bounds.GetBox();
		const FVector4f CustomMaterialData = InComponent->GetCustomMaterialData();
		const FLinearColor FixedColor = InComponent->GetStreamingMipsFixedColor();

		FVTProducerDescription VTDesc;
		RuntimeVirtualTexture->GetProducerDescription(VTDesc, URuntimeVirtualTexture::FInitSettings(), Transform);

		const int32 TileSize = VTDesc.TileSize;
		const int32 TileBorderSize = VTDesc.TileBorderSize;
		const int32 TextureSizeX = VTDesc.WidthInBlocks * VTDesc.BlockWidthInTiles * TileSize;
		const int32 TextureSizeY = VTDesc.HeightInBlocks * VTDesc.BlockHeightInTiles * TileSize;
		const int32 MaxLevel = (int32)FMath::CeilLogTwo(FMath::Max(VTDesc.BlockWidthInTiles, VTDesc.BlockHeightInTiles));
		const int32 RenderLevel = FMath::Max(MaxLevel - InComponent->NumStreamingMips() + 1, 0);
		const int32 ImageSizeX = FMath::Max(TileSize, TextureSizeX >> RenderLevel);
		const int32 ImageSizeY = FMath::Max(TileSize, TextureSizeY >> RenderLevel);
		const int32 NumTilesX = ImageSizeX / TileSize;
		const int32 NumTilesY = ImageSizeY / TileSize;
		const int32 NumLayers = RuntimeVirtualTexture->GetLayerCount();

		const ERuntimeVirtualTextureMaterialType MaterialType = RuntimeVirtualTexture->GetMaterialType();
		TArray<EPixelFormat, TInlineAllocator<4>> LayerFormats;
		for (int32 Layer = 0; Layer < NumLayers; ++Layer)
		{
			LayerFormats.Add(RuntimeVirtualTexture->GetLayerFormat(Layer));
		}

		// Spin up slow task UI
		const float TaskWorkRender = static_cast<float>(NumTilesX * NumTilesY);
		const float TextureBuildTaskMultiplier = 0.25f;
		const float TaskWorkBuildBulkData = TaskWorkRender * TextureBuildTaskMultiplier;
		FScopedSlowTask Task(TaskWorkRender + TaskWorkBuildBulkData, FText::AsCultureInvariant(InComponent->GetStreamingTexture()->GetName()));
		Task.MakeDialog(true);

		// Allocate render targets for rendering out the runtime virtual texture tiles
		FTileRenderResources RenderTileResources(TileSize, NumTilesX, NumTilesY, NumLayers, LayerFormats);
		BeginInitResource(&RenderTileResources);

		int64 RenderTileResourcesBytes = RenderTileResources.GetTotalSizeBytes();

		UE_LOGF(LogVirtualTexturing, Display, "Allocating %uMiB for RenderTileResourcesBytes",(uint32)(RenderTileResourcesBytes/(1024*1024)));

		// Final pixels will contain image data for each virtual texture layer in order
		TArray64<uint8> FinalPixels;
		FinalPixels.SetNumUninitialized(RenderTileResourcesBytes);

		UE::RenderCommandPipe::FSyncScope SyncScope;

		// Iterate over all tiles and render/store each one to the final image
		for (int32 TileY = 0; TileY < NumTilesY && !Task.ShouldCancel(); TileY++)
		{
			for (int32 TileX = 0; TileX < NumTilesX; TileX++)
			{
				// Render tile
				Task.EnterProgressFrame();

				const FBox2D UVRange = FBox2D(
					FVector2D((float)TileX / (float)NumTilesX, (float)TileY / (float)NumTilesY),
					FVector2D((float)(TileX + 1) / (float)NumTilesX, (float)(TileY + 1) / (float)NumTilesY));

				// Stream textures for this tile. This triggers a render flush internally.
				//todo[vt]: Batch groups of streaming locations and render commands to reduce number of flushes.
				const FVector StreamingWorldPos = Transform.TransformPosition(FVector(UVRange.GetCenter(), 0.5f));
				IStreamingManager::Get().Tick(0.f);
				IStreamingManager::Get().AddViewLocation(StreamingWorldPos);
				IStreamingManager::Get().StreamAllResources(0);

				ENQUEUE_RENDER_COMMAND(BakeStreamingTextureTileCommand)([
					Scene, RuntimeVirtualTextureId, 
					&RenderTileResources,
					MaterialType, NumLayers,
					Transform, Bounds, UVRange,
					RenderLevel, MaxLevel, 
					TileX, TileY,
					TileSize, ImageSizeX, ImageSizeY, 
					&FinalPixels,
					CustomMaterialData, FixedColor] (FRHICommandListImmediate& RHICmdList)
				{
					const FIntRect TileRect(0, 0, TileSize, TileSize);

					{
						FRDGBuilder GraphBuilder(RHICmdList);
						FScenePrimitiveRenderingContextScopeHelper RenderingScope(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, *Scene));

						RuntimeVirtualTexture::FRenderPageBatchDesc Desc;
						Desc.SceneRenderer = RenderingScope.ScenePrimitiveRenderingContext->GetSceneRenderer();
						Desc.RuntimeVirtualTextureId = RuntimeVirtualTextureId;
						Desc.UVToWorld = Transform;
						Desc.WorldBounds = Bounds;
						Desc.MaterialType = MaterialType;
						Desc.MaxLevel = IntCastChecked<uint8>(MaxLevel);
						Desc.bClearTextures = true;
						Desc.bIsThumbnails = false;
						Desc.FixedColor = FixedColor;
						Desc.CustomMaterialData = CustomMaterialData;
						Desc.NumPageDescs = 1;
						for (int32 Layer = 0; Layer < NumLayers; Layer++)
						{
							Desc.Targets[Layer].PooledRenderTarget = RenderTileResources.GetRenderTarget(Layer);
							Desc.PageDescs[0].DestRect[Layer] = TileRect;
						}
						Desc.PageDescs[0].UVRange = UVRange;
						Desc.PageDescs[0].vLevel = IntCastChecked<uint8>(RenderLevel);

						RuntimeVirtualTexture::RenderPages(GraphBuilder, Desc);

						// Copy to staging
						for (int32 Layer = 0; Layer < NumLayers; Layer++)
						{
							FRDGTextureRef RenderTarget = GraphBuilder.RegisterExternalTexture(RenderTileResources.GetRenderTarget(Layer), ERDGTextureFlags::None);
							FRDGTextureRef StagingTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(RenderTileResources.GetStagingTexture(Layer), TEXT("StagingTexture")));
							AddCopyTexturePass(GraphBuilder, RenderTarget, StagingTexture, FRHICopyTextureInfo());
						}

						GraphBuilder.Execute();
					}

					RenderTileResources.GetFence()->Clear();
					RHICmdList.WriteGPUFence(RenderTileResources.GetFence());

					// Read back tile data and copy into final destination
					for (int32 Layer = 0; Layer < NumLayers; Layer++)
					{
						void* TilePixels = nullptr;
						int32 OutWidth, OutHeight;
						RHICmdList.MapStagingSurface(RenderTileResources.GetStagingTexture(Layer), RenderTileResources.GetFence(), TilePixels, OutWidth, OutHeight);
						check(TilePixels != nullptr);
						check(OutHeight == TileSize);

						const int64 LayerOffset = RenderTileResources.GetLayerOffset(Layer);
						const EPixelFormat LayerFormat = RenderTileResources.GetLayerFormat(Layer);
						const FIntPoint DestPos(TileX * TileSize, TileY * TileSize);

						CopyTile(TilePixels, TileSize, OutWidth, FinalPixels.GetData() + LayerOffset, ImageSizeX, ImageSizeX * ImageSizeY, DestPos, LayerFormat);

						RHICmdList.UnmapStagingSurface(RenderTileResources.GetStagingTexture(Layer));
					}
				});
			}
		}

		BeginReleaseResource(&RenderTileResources);
		FlushRenderingCommands();

		if (Task.ShouldCancel())
		{
			return false;
		}

		// Place final pixel data into the runtime virtual texture
		Task.EnterProgressFrame(TaskWorkBuildBulkData);

		InComponent->InitializeStreamingTexture(ShadingPath, ImageSizeX, ImageSizeY, (uint8*)FinalPixels.GetData());

		return true;
	}

	FBuildAllStreamedMipsResult BuildAllStreamedMips(const FBuildAllStreamedMipsParams& InParams)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildAllStreamedMips);

		FBuildAllStreamedMipsResult Result;

		// We will need to build VTs for both shading paths
		const ERHIFeatureLevel::Type CurFeatureLevel = InParams.World->GetFeatureLevel();
		const ERHIFeatureLevel::Type AltFeatureLevel = (CurFeatureLevel == ERHIFeatureLevel::ES3_1 ? GMaxRHIFeatureLevel : ERHIFeatureLevel::ES3_1);
		const EShadingPath CurShadingPath = FSceneInterface::GetShadingPath(CurFeatureLevel);
		const EShadingPath AltShadingPath = FSceneInterface::GetShadingPath(AltFeatureLevel);

		TArray<URuntimeVirtualTextureComponent*> Components[2];
		for (URuntimeVirtualTextureComponent* Component : InParams.Components)
		{
			check(Component->GetWorld() == InParams.World);
			if (Component->IsRegistered())
			{
				if (HasStreamedMips(CurShadingPath, Component))
				{
					Components[0].Add(Component);
				}

				if (HasStreamedMips(AltShadingPath, Component))
				{
					Components[1].Add(Component);
				}
			}
			else
			{
				UE_LOGF(LogVirtualTexturing, Error, "Trying to build streaming mips for a component (from actor %ls) that is not registered. It will be ignored", *Component->GetOwner()->GetActorNameOrLabel());
			}
		}

		const int32 NumStreamedMips = Components[0].Num() + Components[1].Num();
		const int32 NumSteps = 
			/* Initial flush = */1 
			+ NumStreamedMips 
			/* Switch to alternate feature level and back = */ + (Components[1].IsEmpty() ? 0 : 2);

		{
			FScopedSlowTask Task(static_cast<float>(NumSteps), LOCTEXT("BuildingStreamingMips", "Building Streamed Mips"));
			Task.MakeDialog(true);

			{
				// Initial flush : 
				Task.EnterProgressFrame();

				// Make sure the World is fully streamed in and ready to render at the current feature level :
				InParams.World->FlushLevelStreaming();
				FAssetCompilingManager::Get().FinishAllCompilation();

				// Recreate render state after shader compilation complete
				{
					FGlobalComponentRecreateRenderStateContext Context;
				}
			}

			// Build for a current feature level first
			if (Components[0].Num() != 0)
			{
				for (URuntimeVirtualTextureComponent* Component : Components[0])
				{
					if (Task.ShouldCancel())
					{
						Result.bSuccess = false;
						break;
					}

					Task.EnterProgressFrame();
					bool bSuccess = BuildStreamedMips(CurShadingPath, Component);
					if (bSuccess)
					{
						if (UVirtualTextureBuilder* VTBuilder = Component->GetStreamingTexture(); VTBuilder->GetPackage()->IsDirty())
						{
							Result.ModifiedPackages.Add(VTBuilder->GetPackage());
						}
					}

					Result.bSuccess &= bSuccess;
				}
			}

			// Build for others if any
			if ((Components[1].Num() != 0) && !Task.ShouldCancel())
			{
				{
					// Setup alternate feature level : 
					Task.EnterProgressFrame();

					// Commandlets do not initialize shader resources for alternate feature levels, do it now
					{
						bool bUpdateProgressDialog = false;
						bool bCacheAllRemainingShaders = true;
						UMaterialInterface::SetGlobalRequiredFeatureLevel(AltFeatureLevel, true);
						UMaterial::AllMaterialsCacheResourceShadersForRendering(bUpdateProgressDialog, bCacheAllRemainingShaders);
						UMaterialInstance::AllMaterialsCacheResourceShadersForRendering(bUpdateProgressDialog, bCacheAllRemainingShaders);
						CompileGlobalShaderMap(AltFeatureLevel);
					}

					InParams.World->ShaderPlatformChanged(GShaderPlatformForFeatureLevel[AltFeatureLevel]);

					// Make sure all assets are finished compiling. Recreate render state after shader compilation complete
					{
						UMaterialInterface::SubmitRemainingJobsForWorld(InParams.World);
						FAssetCompilingManager::Get().FinishAllCompilation();
						FAssetCompilingManager::Get().ProcessAsyncTasks();
						FGlobalComponentRecreateRenderStateContext Context;
					}

					// Flush all rendering commands issued by UpdateAllPrimitiveSceneInfos inside the FGlobalComponentRecreateRenderStateContext. 
					// Some rendering commands may trigger some shader compilations that we need to be issued and wait for completion before rendering the RVT.
					FlushRenderingCommands();

					// FGlobalComponentRecreateRenderStateContext can create new shaderJobs, make sure to wait on them.
					FAssetCompilingManager::Get().FinishAllCompilation();
					FAssetCompilingManager::Get().ProcessAsyncTasks();
				}

				for (URuntimeVirtualTextureComponent* Component : Components[1])
				{
					if (Task.ShouldCancel())
					{
						Result.bSuccess = false;
						break;
					}

					Task.EnterProgressFrame();
					bool bSuccess = BuildStreamedMips(AltShadingPath, Component);
					if (bSuccess)
					{
						if (UVirtualTextureBuilder* VTBuilder = Component->GetStreamingTexture(); VTBuilder->GetPackage()->IsDirty())
						{
							Result.ModifiedPackages.Add(VTBuilder->GetPackage());
						}
					}

					Result.bSuccess &= bSuccess;
				}

				// Restore world feature level
				if (InParams.bRestoreFeatureLevelAfterBuilding)
				{
					Task.EnterProgressFrame();
					UMaterialInterface::SetGlobalRequiredFeatureLevel(CurFeatureLevel, /*bShouldCompile = */ false);
					InParams.World->ShaderPlatformChanged(GShaderPlatformForFeatureLevel[CurFeatureLevel]);
				}
			}
		}

		return MoveTemp(Result);
	}
}

#undef LOCTEXT_NAMESPACE