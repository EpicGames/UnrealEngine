// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "VirtualTexturingEditorModule.h"

class URuntimeVirtualTextureComponent;
class UWorld;
enum class EShadingPath;

namespace RuntimeVirtualTexture
{
	/** Returns true if the component describes a runtime virtual texture that has streaming mips. */
	bool HasStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent);

	/** Build the streaming mips texture. */
	bool BuildStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent);

	FBuildAllStreamedMipsResult BuildAllStreamedMips(const FBuildAllStreamedMipsParams& InParams);
};

