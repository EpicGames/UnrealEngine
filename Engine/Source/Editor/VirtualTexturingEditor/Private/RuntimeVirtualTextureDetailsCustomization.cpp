// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeVirtualTextureDetailsCustomization.h"

#include "AssetToolsModule.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/Texture2D.h"
#include "Factories/Texture2dFactoryNew.h"
#include "RuntimeVirtualTextureBuildStreamingMips.h"
#include "RuntimeVirtualTextureSetBounds.h"
#include "ScopedTransaction.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "SEnumCombo.h"
#include "SResetToDefaultMenu.h"
#include "VirtualTextureBuilderFactory.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/VirtualTextureBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "VirtualTexturingEditorModule"

FRuntimeVirtualTextureDetailsCustomization::FRuntimeVirtualTextureDetailsCustomization()
{
}

TSharedRef<IDetailCustomization> FRuntimeVirtualTextureDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FRuntimeVirtualTextureDetailsCustomization);
}

namespace
{
	// Helper for adding text containing real values to the properties that are edited as power (or multiple) of 2
	void AddTextToProperty(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& CategoryBuilder, FName const& PropertyName, TSharedPtr<STextBlock>& TextBlock)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(PropertyName);
		DetailBuilder.HideProperty(PropertyHandle);

		TSharedPtr<SResetToDefaultMenu> ResetToDefaultMenu;

		CategoryBuilder.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)

				+ SWrapBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 2.0f, 0.0f))
				[
					SAssignNew(TextBlock, STextBlock)
				]
			]

			+ SHorizontalBox::Slot()
			[
				PropertyHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f)
			[
				// Would be better to use SResetToDefaultPropertyEditor here but that is private in the PropertyEditor lib
				SAssignNew(ResetToDefaultMenu, SResetToDefaultMenu)
			]
		];

		ResetToDefaultMenu->AddProperty(PropertyHandle.ToSharedRef());
	}
}

void FRuntimeVirtualTextureDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Get and store the linked URuntimeVirtualTexture
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	VirtualTexture = Cast<URuntimeVirtualTexture>(ObjectsBeingCustomized[0].Get());
	if (VirtualTexture == nullptr)
	{
		return;
	}

	RefreshMaterialTypes();

	TSharedRef<IPropertyHandle> MaterialTypePropertyHandle = DetailBuilder.GetProperty(TEXT("MaterialType"));
	DetailBuilder.EditDefaultProperty(MaterialTypePropertyHandle)->CustomWidget()
		.NameContent()
		[
			MaterialTypePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEnumComboBox, StaticEnum<ERuntimeVirtualTextureMaterialType>())
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.EnumValueSubset(SupportedMaterialTypes)
				.CurrentValue_Lambda([this]()
				{
					if (URuntimeVirtualTexture* Texture = VirtualTexture.Get())
					{
						return (int32)Texture->GetMaterialType();
					}
					return 0;
				})
				.OnEnumSelectionChanged_Lambda([this](uint32 NewValue, ESelectInfo::Type)
				{
					if (URuntimeVirtualTexture* Texture = VirtualTexture.Get())
					{
						Texture->MaterialType = (ERuntimeVirtualTextureMaterialType)NewValue;
						RefreshDetailsView();
					}
				})
		];

	// Set UIMax dependent on adaptive page table setting
	FString MaxTileCountString = FString::Printf(TEXT("%d"), URuntimeVirtualTexture::GetMaxTileCountLog2(VirtualTexture->GetAdaptivePageTable()));
	DetailBuilder.GetProperty(FName(TEXT("TileCount")))->SetInstanceMetaData("UIMax", MaxTileCountString);

	// Add size helpers
	IDetailCategoryBuilder& SizeCategory = DetailBuilder.EditCategory("Size", FText::GetEmpty());
	AddTextToProperty(DetailBuilder, SizeCategory, "TileCount", TileCountText);
	AddTextToProperty(DetailBuilder, SizeCategory, "TileSize", TileSizeText);
	AddTextToProperty(DetailBuilder, SizeCategory, "TileBorderSize", TileBorderSizeText);

	// Add details block
	{
		IDetailCategoryBuilder& DetailsCategory = DetailBuilder.EditCategory("Details", FText::GetEmpty(), ECategoryPriority::Important);
		static const FText CustomRowSizeText = LOCTEXT("Details_RowFilter_Size", "Virtual Size");
		DetailsCategory.AddCustomRow(CustomRowSizeText)
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Details_Size", "Virtual Texture Size"))
				.ToolTipText(LOCTEXT("Details_Size_Tooltip", "Virtual resolution derived from Size properties."))
			]
			.ValueContent()
			[
				SAssignNew(SizeText, STextBlock)
			];
		static const FText CustomRowPageTableSizeText = LOCTEXT("Details_RowFilter_PageTableSize", "Page Table Size");
		DetailsCategory.AddCustomRow(CustomRowPageTableSizeText)
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Details_PageTableSize", "Page Table Size"))
				.ToolTipText(LOCTEXT("Details_PageTableSize_Tooltip", "Final page table size. This can vary according to the adaptive page table setting."))
			]
			.ValueContent()
			[
				SAssignNew(PageTableSizeText, STextBlock)
			];
	}

	// Priority property :
	{
		IDetailCategoryBuilder& PerformanceCategory = DetailBuilder.EditCategory("Performance", FText::GetEmpty());
		static const FText CurrentPriorityText = LOCTEXT("CurrentPriority", "Current Priority");
		// Add the current priority before the custom priority : 
		TSharedRef<IPropertyHandle> CustomPriorityHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(URuntimeVirtualTexture, CustomPriority));
		CustomPriorityHandle->MarkHiddenByCustomization();
		PerformanceCategory.AddCustomRow(/*FilterString = */CurrentPriorityText, /*bForAdvanced = */true)
			.NameContent()
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(CurrentPriorityText)
					.IsEnabled(false)
					.ToolTipText(LOCTEXT("CurrentPriority_Tooltip", "Defines the relative priority that this Runtime Virtual Texture has relative to other virtual texture producers.\n"
						"This allows to get the pages from this virtual texture to update faster than others in case of high contention.\n"
						"By default, it is derived from the material type but can be overridden with the Custom Priority property."))
			]
			.ValueContent()
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text_Lambda([this]()
						{
							if (URuntimeVirtualTexture* Texture = VirtualTexture.Get())
							{
								return UEnum::GetDisplayValueAsText(Texture->GetPriority());
							}
							return FText();
						})
					.ToolTipText_Lambda([this]()
						{
							if (URuntimeVirtualTexture* Texture = VirtualTexture.Get())
							{
								return Texture->GetUseCustomPriority() ? 
									LOCTEXT("CurrentPriorityCustom_Tooltip", "Custom Priority") :
									LOCTEXT("CurrentPriorityDefault_Tooltip", "Default Priority (inferred from the virtual texture content property)");
							}
							return FText();
						})
			];
		PerformanceCategory.AddProperty(CustomPriorityHandle, EPropertyLocation::Advanced);
	}

	// Cache detail builder to refresh view updates
	CachedDetailBuilder = &DetailBuilder;

	// Add refresh callback for all properties 
	DetailBuilder.GetProperty(FName(TEXT("TileCount")))->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRuntimeVirtualTextureDetailsCustomization::RefreshTextDetails));
	DetailBuilder.GetProperty(FName(TEXT("TileSize")))->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRuntimeVirtualTextureDetailsCustomization::RefreshTextDetails));
	DetailBuilder.GetProperty(FName(TEXT("TileBorderSize")))->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRuntimeVirtualTextureDetailsCustomization::RefreshTextDetails));
	DetailBuilder.GetProperty(FName(TEXT("bAdaptive")))->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRuntimeVirtualTextureDetailsCustomization::RefreshDetailsView));

	// Initialize text blocks
	RefreshTextDetails();
}

void FRuntimeVirtualTextureDetailsCustomization::RefreshMaterialTypes()
{
	// Filter for enabled material types.
	SupportedMaterialTypes.Reset();
	SupportedMaterialTypes.Reserve((int32)ERuntimeVirtualTextureMaterialType::Count);
	
	// Include currently selected type even if it is disabled.
	ERuntimeVirtualTextureMaterialType CurrentType = ERuntimeVirtualTextureMaterialType::Count;
	if (URuntimeVirtualTexture* Texture = VirtualTexture.Get())
	{
		CurrentType = Texture->GetMaterialType();
	}

	for (ERuntimeVirtualTextureMaterialType Type : TEnumRange<ERuntimeVirtualTextureMaterialType>())
	{
		if (RuntimeVirtualTexture::IsMaterialTypeSupported(Type) || Type == CurrentType)
		{
			SupportedMaterialTypes.Add((int32)Type);
		}
	}
}

void FRuntimeVirtualTextureDetailsCustomization::RefreshTextDetails()
{
	if (URuntimeVirtualTexture* Texture = VirtualTexture.Get())
	{
		FNumberFormattingOptions SizeOptions;
		SizeOptions.UseGrouping = false;
		SizeOptions.MaximumFractionalDigits = 0;

		TileCountText->SetText(FText::Format(LOCTEXT("Details_Number", "{0}"), FText::AsNumber(Texture->GetTileCount(), &SizeOptions)));
		TileSizeText->SetText(FText::Format(LOCTEXT("Details_Number", "{0}"), FText::AsNumber(Texture->GetTileSize(), &SizeOptions)));
		TileBorderSizeText->SetText(FText::Format(LOCTEXT("Details_Number", "{0}"), FText::AsNumber(Texture->GetTileBorderSize(), &SizeOptions)));

		FString SizeUnits = TEXT("Texels");
		int32 Size = Texture->GetSize();
		int32 SizeLog2 = FMath::CeilLogTwo(Size);
		if (SizeLog2 >= 30)
		{
			Size = Size >> 30;
			SizeUnits = TEXT("GiTexels");
		}
		else if (SizeLog2 >= 20)
		{
			Size = Size >> 20;
			SizeUnits = TEXT("MiTexels");
		}
		else if (SizeLog2 >= 10)
		{
			Size = Size >> 10;
			SizeUnits = TEXT("KiTexels");
		}
		SizeText->SetText(FText::Format(LOCTEXT("Details_Number_Units", "{0} {1}"), FText::AsNumber(Size, &SizeOptions), FText::FromString(SizeUnits)));

		PageTableSizeText->SetText(FText::Format(LOCTEXT("Details_Number", "{0}"), FText::AsNumber(Texture->GetPageTableSize(), &SizeOptions)));
	}
}

void FRuntimeVirtualTextureDetailsCustomization::RefreshDetailsView()
{
	if (CachedDetailBuilder != nullptr)
	{
		CachedDetailBuilder->ForceRefreshDetails();
	}
}


FRuntimeVirtualTextureComponentDetailsCustomization::FRuntimeVirtualTextureComponentDetailsCustomization()
{
}

TSharedRef<IDetailCustomization> FRuntimeVirtualTextureComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FRuntimeVirtualTextureComponentDetailsCustomization);
}

void FRuntimeVirtualTextureComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Get and store the linked URuntimeVirtualTextureComponent.
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	RuntimeVirtualTextureComponent = Cast<URuntimeVirtualTextureComponent>(ObjectsBeingCustomized[0].Get());
	if (RuntimeVirtualTextureComponent == nullptr)
	{
		return;
	}

	// Apply custom widget for SetBounds.
	TSharedRef<IPropertyHandle> SetBoundsPropertyHandle = DetailBuilder.GetProperty(TEXT("bSetBoundsButton"));
	DetailBuilder.EditDefaultProperty(SetBoundsPropertyHandle)->CustomWidget()
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("Button_SetBounds", "Set Bounds"))
		.ToolTipText(LOCTEXT("Button_SetBounds_Tooltip", "Set the rotation to match the Bounds Align Actor and expand bounds to include all primitives that write to this virtual texture."))
	]
	.ValueContent()
	.MinDesiredWidth(125.f)
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.ContentPadding(2.f)
		.Text(LOCTEXT("Button_SetBounds", "Set Bounds"))
		.OnClicked(this, &FRuntimeVirtualTextureComponentDetailsCustomization::SetBounds)
		.IsEnabled(this, &FRuntimeVirtualTextureComponentDetailsCustomization::IsSetBoundsEnabled)
	];

	static const FText BuildButtonSuffixText = LOCTEXT("Button_Build_Suffix_Tooltip", "Use the Build / Build Streaming Runtime Virtual Textures menu to build all versions of all Runtime Virtual Texture Volumes, after ensuring all actors rendering to RVT are loaded.");

	// Apply custom widget for BuildStreamingMips.
	TSharedRef<IPropertyHandle> BuildStreamingMipsPropertyHandle = DetailBuilder.GetProperty(TEXT("bBuildStreamingMipsButton"));
	DetailBuilder.EditDefaultProperty(BuildStreamingMipsPropertyHandle)->CustomWidget()
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("Button_BuildStreamingTexture", "Build Streaming Texture"))
		.ToolTipText(FText::Format(LOCTEXT("Button_Build_Prefix_Tooltip", "Build runtime virtual texture low mips streaming data.\n{0}"), BuildButtonSuffixText))
	]
	.ValueContent()
	.MinDesiredWidth(150.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(4.0f)
		.MinWidth(100.0f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ContentPadding(2.f)
			.Text_Lambda([this]()
				{
					return IsStreamingTextureSet() ? LOCTEXT("Button_Build", "Build") : LOCTEXT("Button_CreateAndBuild", "Create & Build");
				})
			.ToolTipText(FText::Format(LOCTEXT("Button_Build_Tooltip", "Build the low mips as streaming virtual texture data for the current shading path, using the currently loaded actors. \n"
				"If \"Separate Texture For Mobile\" is enabled in the Streaming Texture, only the mobile version of the texture will be updated when using this button "
				"while the mobile preview mode is active (and only the desktop version otherwise).\n{0}"), BuildButtonSuffixText))
			.OnClicked(this, &FRuntimeVirtualTextureComponentDetailsCustomization::BuildStreamedMips, /*bBuildAll = */false)
			.IsEnabled(this, &FRuntimeVirtualTextureComponentDetailsCustomization::IsBuildStreamedMipsEnabled)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(4.0f)
		.MinWidth(100.0f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ContentPadding(2.f)
			.Text(LOCTEXT("Button_BuildAll", "Build All"))
			.ToolTipText(FText::Format(LOCTEXT("Button_BuildAll_Tooltip", "Build the low mips as streaming virtual texture data for all shading paths, if necessary, using the currently loaded actors. \n{0}"), BuildButtonSuffixText))
			.OnClicked(this, &FRuntimeVirtualTextureComponentDetailsCustomization::BuildStreamedMips, /*bBuildAll = */true)
			.IsEnabled(this, &FRuntimeVirtualTextureComponentDetailsCustomization::IsBuildStreamedMipsEnabled)
			.Visibility(this, &FRuntimeVirtualTextureComponentDetailsCustomization::GetBuildAllStreamedMipsVisible)
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
			.Visibility_Lambda([this]()
				{
					FText Dummy;
					return IsStreamingTextureValid(Dummy) ? EVisibility::Hidden : EVisibility::Visible;
				})
			.ToolTipText_Lambda([this]() 
			{
				FText Reason;
				if (!IsStreamingTextureValid(Reason))
				{
					const FText BuildAllShadingPathsText = LOCTEXT("BuildAllShadingPaths", " or Build All to rebuild it for all shading paths");
					return FText::Format(LOCTEXT("Warning_Build_Tooltip", "The settings have changed since the Streaming Texture was last rebuilt.\n{0}\n"
						"Use the Build button to rebuild the streaming virtual texture for the current shading path{1}.\n"
						"Build Streaming Runtime Virtual Textures in the Build menu can also be used to rebuild all loaded streaming runtime virtual textures for all shading paths, after ensuring all actors rendering to RVT are loaded.\n"
						"Meanwhile, streaming mips will be disabled."), 
						Reason, (GetBuildAllStreamedMipsVisible() == EVisibility::Visible) ? BuildAllShadingPathsText : FText());
				}
				return FText();
			})
		]
	];
}

bool FRuntimeVirtualTextureComponentDetailsCustomization::IsSetBoundsEnabled() const
{
	if (URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get())
	{
		return Component->GetVirtualTexture() != nullptr;
	}
	return false;
}

FReply FRuntimeVirtualTextureComponentDetailsCustomization::SetBounds()
{
	if (URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get())
	{
		if (Component->GetVirtualTexture() != nullptr)
		{
			const FScopedTransaction Transaction(LOCTEXT("Transaction_SetBounds", "Set RuntimeVirtualTextureComponent Bounds"));
			RuntimeVirtualTexture::SetBounds(Component);
			// Force update of editor view widget.
			GEditor->NoteSelectionChange(false);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

bool FRuntimeVirtualTextureComponentDetailsCustomization::IsBuildStreamedMipsEnabled() const
{
	if (URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get())
	{
		return Component->GetVirtualTexture() != nullptr && Component->NumStreamingMips() > 0;
	}
	return false;
}

EVisibility FRuntimeVirtualTextureComponentDetailsCustomization::GetBuildAllStreamedMipsVisible() const
{
	if (URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get())
	{
		UVirtualTextureBuilder* StreamedMipsTexture = Component->GetStreamingTexture();
		// If there is a virtual texture to build for another shading path, then Build All makes sense
		if (IsBuildStreamedMipsEnabled() && (StreamedMipsTexture != nullptr) && StreamedMipsTexture->bSeparateTextureForMobile)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

bool FRuntimeVirtualTextureComponentDetailsCustomization::IsStreamingTextureSet() const
{
	URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get();
	return Component && (Component->GetStreamingTexture() != nullptr);
}

bool FRuntimeVirtualTextureComponentDetailsCustomization::IsStreamingTextureValid(FText& OutReason) const
{
	using EStreamingTextureStatusFlags = URuntimeVirtualTextureComponent::EStreamingTextureStatusFlags;

	if (URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get())
	{
		if (RuntimeVirtualTextureComponent->IsStreamingTextureInvalid())
		{
			FTextBuilder TextBuilder;
			auto AppendError = [&TextBuilder](EStreamingTextureStatusFlags InStatus, const FText& InStreamingTextureName)
				{
					if (EnumHasAllFlags(InStatus, EStreamingTextureStatusFlags::HasVirtualTexture | EStreamingTextureStatusFlags::HasStreamingTexture)
						&& EnumHasAnyFlags(InStatus, EStreamingTextureStatusFlags::InvalidStreamingTexture))
					{
						TextBuilder.AppendLineFormat(LOCTEXT("UnbuiltStreamingTexture_Reason", "The {0} is setup but not built yet."), InStreamingTextureName);
					}
					if (EnumHasAllFlags(InStatus, EStreamingTextureStatusFlags::HasVirtualTexture | EStreamingTextureStatusFlags::HasStreamingTexture)
						&& EnumHasAnyFlags(InStatus, EStreamingTextureStatusFlags::NonMatchingStreamingTextureSettings))
					{
						TextBuilder.AppendLineFormat(LOCTEXT("NotUpToDateStreamingTexture_Reason", "The {0} is not up to date."), InStreamingTextureName);
					}
				};
			AppendError(RuntimeVirtualTextureComponent->GetStreamingTextureStatus(EShadingPath::Deferred), LOCTEXT("StreamingTexture", "streaming texture"));
			AppendError(RuntimeVirtualTextureComponent->GetStreamingTextureStatus(EShadingPath::Mobile), LOCTEXT("MobileStreamingTexture", "mobile streaming texture"));
			OutReason = TextBuilder.ToText();
			return false;
		}
	}
	return true;
}

FReply FRuntimeVirtualTextureComponentDetailsCustomization::BuildStreamedMips(bool bBuildAll)
{
	if (URuntimeVirtualTextureComponent* Component = RuntimeVirtualTextureComponent.Get())
	{
		UWorld* World = Component->GetWorld();
		check(World != nullptr);

		// Create a new asset if none is already bound
		UVirtualTextureBuilder* CreatedTexture = nullptr;
		if (Component->GetVirtualTexture() != nullptr && Component->GetStreamingTexture() == nullptr)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

			const FString DefaultPath = FPackageName::GetLongPackagePath(Component->GetVirtualTexture()->GetPathName());
			const FString DefaultName = FPackageName::GetShortName(Component->GetVirtualTexture()->GetName() + TEXT("_SVT"));

			UFactory* Factory = NewObject<UVirtualTextureBuilderFactory>();
			UObject* Object = AssetToolsModule.Get().CreateAssetWithDialog(DefaultName, DefaultPath, UVirtualTextureBuilder::StaticClass(), Factory);
			CreatedTexture = Cast<UVirtualTextureBuilder>(Object);
		}

		// Build the texture contents
		bool bOK = false;
		if (Component->GetStreamingTexture() != nullptr || CreatedTexture != nullptr)
		{
			const FScopedTransaction Transaction(LOCTEXT("Transaction_BuildDebugStreamingTexture", "Build Streaming Texture"));

			if (CreatedTexture != nullptr)
			{
				Component->Modify();
				Component->SetStreamingTexture(CreatedTexture);
			}

			Component->GetStreamingTexture()->Modify();

			if (bBuildAll)
			{
				FBuildAllStreamedMipsResult Result = RuntimeVirtualTexture::BuildAllStreamedMips({ .World = World, .Components = { Component }, .bRestoreFeatureLevelAfterBuilding = true });
				bOK = Result.bSuccess;
			}
			else
			{
				const ERHIFeatureLevel::Type CurFeatureLevel = World->GetFeatureLevel();
				const EShadingPath CurShadingPath = FSceneInterface::GetShadingPath(CurFeatureLevel);
				bOK = RuntimeVirtualTexture::BuildStreamedMips(CurShadingPath, Component);
			}
		}

		return bOK ? FReply::Handled() : FReply::Unhandled();
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
