// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeVirtualTextureFactory.h"

#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimeVirtualTextureFactory)

class FFeedbackContext;
class UClass;
class UObject;

URuntimeVirtualTextureFactory::URuntimeVirtualTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = URuntimeVirtualTexture::StaticClass();
	
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;
}

UObject* URuntimeVirtualTextureFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	URuntimeVirtualTexture* VirtualTexture = NewObject<URuntimeVirtualTexture>(InParent, Class, Name, Flags);
	check(VirtualTexture);
	return VirtualTexture;
}
