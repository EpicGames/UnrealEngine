// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeVirtualTextureSetBounds.h"

#include "Components/PrimitiveComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "GameFramework/Actor.h"
#include "Landscape.h"
#include "LandscapeInfo.h"
#include "UObject/UObjectIterator.h"
#include "VT/RuntimeVirtualTexture.h"

namespace RuntimeVirtualTexture
{
	void SetBounds(URuntimeVirtualTextureComponent* InComponent)
	{
		URuntimeVirtualTexture const* VirtualTexture = InComponent->GetVirtualTexture();
		check(VirtualTexture != nullptr);

		// Calculate bounds in our desired local space.
		AActor* Owner = InComponent->GetOwner();
		const FVector TargetPosition = Owner->ActorToWorld().GetTranslation();
		
		// Local space will take rotation from a BoundsAlignActor if set.
		TSoftObjectPtr<AActor>& BoundsAlignActor = InComponent->GetBoundsAlignActor();
		const FQuat TargetRotation = BoundsAlignActor.IsValid() ? BoundsAlignActor->GetTransform().GetRotation() : Owner->ActorToWorld().GetRotation();

		FTransform LocalTransform;
		LocalTransform.SetComponents(TargetRotation, TargetPosition, FVector::OneVector);
		FTransform WorldToLocal = LocalTransform.Inverse();

		// Expand bounds for the BoundsAlignActor and all primitive components that write to this virtual texture.
		FBox Bounds(ForceInit);

		// Special case where if the bounds align actor is a landscape: we want to automatically include all associated landscape components, 
		//  including those that are not currently loaded. Luckily there's a function for that:
		if (BoundsAlignActor.IsValid())
		{
			if (ALandscape const* Landscape = Cast<ALandscape>(BoundsAlignActor.Get()))
			{
				FBox WorldBounds = Landscape->GetCompleteBounds();
				Bounds = WorldBounds.TransformBy(WorldToLocal);
			}
		}

		for (TObjectIterator<UPrimitiveComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			bool bUseBounds = BoundsAlignActor.IsValid() && It->GetOwner() == BoundsAlignActor.Get();

			TArray<URuntimeVirtualTexture*> const& VirtualTextures = It->GetRuntimeVirtualTextures();
			for (int32 Index = 0; !bUseBounds && Index < VirtualTextures.Num(); ++Index) 
			{
				if (VirtualTextures[Index] == InComponent->GetVirtualTexture())
				{
					bUseBounds = true;
				}
			}

			if (bUseBounds)
			{
				FBoxSphereBounds LocalSpaceBounds = It->CalcBounds(It->GetComponentTransform() * WorldToLocal);
				Bounds += LocalSpaceBounds.GetBox();
			}
		}

		if (Bounds.IsValid)
		{
			const FVector BoundsSize = Bounds.GetSize();
			// If XY bounds are valid but Z is 0, let's just expand by a little value to get something to render still (e.g. flat landscape)
			if ((BoundsSize.X > UE_KINDA_SMALL_NUMBER) && (BoundsSize.Y > UE_KINDA_SMALL_NUMBER) && (BoundsSize.Z <= UE_KINDA_SMALL_NUMBER))
			{
				Bounds = Bounds.ExpandBy(FVector(0.0, 0.0, 0.5));
			}

			// Expand bounds if requested
			const float ExpandBounds = InComponent->GetExpandBounds();
			if (ExpandBounds > 0.0f)
			{
				Bounds = Bounds.ExpandBy(ExpandBounds);
			}
		}

		// Calculate the transform to fit the bounds.
		FTransform Transform;
		const FVector LocalPosition = Bounds.Min;
		const FVector WorldPosition = LocalTransform.TransformPosition(LocalPosition);
		const FVector WorldSize = Bounds.GetSize();
		Transform.SetComponents(TargetRotation, WorldPosition, WorldSize);

		// Adjust and snap to landscape if requested.
		// This places the texels on the landscape vertex positions which is desirable for virtual textures that hold height or position information.
		// Warning: This shifts the virtual texture volume so that it might be larger then the landscape (or smaller if insufficient resolution has been set).
		if (InComponent->GetSnapBoundsToLandscape() && BoundsAlignActor.IsValid())
		{
			ALandscape const* Landscape = Cast<ALandscape>(BoundsAlignActor.Get());
			if (Landscape != nullptr)
			{
				const FTransform LandscapeTransform = Landscape->GetTransform();
				const FVector LandscapePosition = LandscapeTransform.GetTranslation();
				const FVector LandscapeScale = LandscapeTransform.GetScale3D();

				// We want to set the virtual texture scale so that the landscape quad size is some power of two multiple of the final virtual texture texel size.
				ULandscapeInfo const* LandscapeInfo = Landscape->GetLandscapeInfo();
				FIntRect LandscapeRect = LandscapeInfo->GetCompleteLandscapeExtent();
				const FIntPoint LandscapeSize = LandscapeRect.Size();
				const int32 LandscapeSizeLog2 = FMath::Max(FMath::CeilLogTwo(LandscapeSize.X), FMath::CeilLogTwo(LandscapeSize.Y));

				const int32 VirtualTextureSize = VirtualTexture->GetSize();
				const int32 VirtualTextureSizeLog2 = FMath::FloorLog2(VirtualTextureSize);

				const int32 VirtualTexelsPerLandscapeVertexLog2 = FMath::Max(VirtualTextureSizeLog2 - LandscapeSizeLog2, 0);
				const int32 VirtualTexelsPerLandscapeVertex = 1 << VirtualTexelsPerLandscapeVertexLog2;

				FVector VirtualTexelWorldSize = LandscapeScale / (float)VirtualTexelsPerLandscapeVertex;
				FVector VirtualTextureScale = VirtualTexelWorldSize * (float)VirtualTextureSize;

				// We may need to adjust for the case where the calculated VirtualTextureScale wasn't big enough to cover the previously calculated bounds.
				const int32 AdditionalScaleX = (int32)FMath::CeilToInt(WorldSize.X / VirtualTextureScale.X);
				const int32 AdditionalScaleY = (int32)FMath::CeilToInt(WorldSize.Y / VirtualTextureScale.Y);
				const int32 AdditionalScaleAligned = FMath::RoundUpToPowerOfTwo(FMath::Max(AdditionalScaleX, AdditionalScaleY));
				// Need to clamp scale so that virtual texture texel is no bigger than a landscape quad. 
				// This may prevent us from covering the required bounds. In that case the user fix is to increase the resolution of the runtime virtual texture.
				const float AdditionalScale = (float)FMath::Min(VirtualTexelsPerLandscapeVertex, AdditionalScaleAligned);

				VirtualTexelWorldSize.X *= AdditionalScale;
				VirtualTexelWorldSize.Y *= AdditionalScale;
				VirtualTextureScale.X *= AdditionalScale;
				VirtualTextureScale.Y *= AdditionalScale;

				Transform.SetScale3D(FVector(VirtualTextureScale.X, VirtualTextureScale.Y, Transform.GetScale3D().Z));
				
				// Adjust position to snap at a half texel offset from landscape.
				const FVector BaseVirtualTexturePosition = Transform.GetTranslation();
				const FVector LandscapeSnapPosition = LandscapePosition - 0.5f * VirtualTexelWorldSize;
				const double SnapOffsetX = FMath::Frac((BaseVirtualTexturePosition.X - LandscapeSnapPosition.X) / VirtualTexelWorldSize.X) * VirtualTexelWorldSize.X;
				const double SnapOffsetY = FMath::Frac((BaseVirtualTexturePosition.Y - LandscapeSnapPosition.Y) / VirtualTexelWorldSize.Y) * VirtualTexelWorldSize.Y;
				const FVector VirtualTexturePosition = BaseVirtualTexturePosition - FVector(SnapOffsetX, SnapOffsetY, 0);
				Transform.SetTranslation(FVector(BaseVirtualTexturePosition.X - SnapOffsetX, BaseVirtualTexturePosition.Y - SnapOffsetY, BaseVirtualTexturePosition.Z));
			}
		}

		// Apply final result and notify the parent actor
		Owner->Modify();
		Owner->SetActorTransform(Transform);
		Owner->PostEditMove(true);
	}
}
