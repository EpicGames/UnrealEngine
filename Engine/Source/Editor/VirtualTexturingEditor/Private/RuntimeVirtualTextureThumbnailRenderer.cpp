// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeVirtualTextureThumbnailRenderer.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "EngineModule.h"
#include "Engine/World.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "PixelFormat.h"
#include "RHI.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/UObjectIterator.h"
#include "UnrealClient.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "VT/RuntimeVirtualTextureRender.h"
#include "VirtualTextureEnum.h"
#include "VirtualTexturing.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimeVirtualTextureThumbnailRenderer)

class FCanvas;
class FRHICommandListImmediate;

namespace
{
	/** Find a matching component for this URuntimeVirtualTexture. */
	URuntimeVirtualTextureComponent* FindComponent(URuntimeVirtualTexture* RuntimeVirtualTexture)
	{
		for (TObjectIterator<URuntimeVirtualTextureComponent> It; It; ++It)
		{
			URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = *It;
			if (RuntimeVirtualTextureComponent->GetVirtualTexture() == RuntimeVirtualTexture)
			{
				return RuntimeVirtualTextureComponent;
			}
		}

		return nullptr;
	}
}

URuntimeVirtualTextureThumbnailRenderer::URuntimeVirtualTextureThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool URuntimeVirtualTextureThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	URuntimeVirtualTexture* RuntimeVirtualTexture = Cast<URuntimeVirtualTexture>(Object);

	// We need a matching URuntimeVirtualTextureComponent in a Scene to be able to render a thumbnail
	URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = FindComponent(RuntimeVirtualTexture);
	if (RuntimeVirtualTextureComponent != nullptr && RuntimeVirtualTextureComponent->GetScene() != nullptr)
	{
		return true;
	}

	return false;
}

void URuntimeVirtualTextureThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	//todo[vt]: Handle case where a null or floating point render target is passed in. (This happens on package save.)
	if (RenderTarget->GetRenderTargetTexture() == nullptr || RenderTarget->GetRenderTargetTexture()->GetFormat() != PF_B8G8R8A8)
	{
		return;
	}

	URuntimeVirtualTexture* RuntimeVirtualTexture = Cast<URuntimeVirtualTexture>(Object);
	const int32 RuntimeVirtualTextureId = RuntimeVirtualTexture->GetUniqueID();
	URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = FindComponent(RuntimeVirtualTexture);
	FSceneInterface* Scene = RuntimeVirtualTextureComponent != nullptr ? RuntimeVirtualTextureComponent->GetScene() : nullptr;
	check(Scene != nullptr);

	if (UWorld* World = Scene->GetWorld())
	{
		World->SendAllEndOfFrameUpdates();
	}

	const FIntRect DestRect = FIntRect(X, Y, X + Width, Y + Height);
	const FTransform Transform = RuntimeVirtualTextureComponent->GetComponentTransform();
	const FBox Bounds = RuntimeVirtualTextureComponent->Bounds.GetBox();
	const ERuntimeVirtualTextureMaterialType MaterialType = RuntimeVirtualTexture->GetMaterialType();
	const FVector4f CustomMaterialData = RuntimeVirtualTextureComponent->GetCustomMaterialData();

	FVTProducerDescription VTDesc;
	RuntimeVirtualTexture->GetProducerDescription(VTDesc, URuntimeVirtualTexture::FInitSettings(), Transform);
	// No need for thumbnails to be produced before anything else : 
	VTDesc.Priority = EVTProducerPriority::Lowest;
	const int32 MaxLevel = (int32)FMath::CeilLogTwo(FMath::Max(VTDesc.BlockWidthInTiles, VTDesc.BlockHeightInTiles));

	UE::RenderCommandPipe::FSyncScope SyncScope;

	ENQUEUE_RENDER_COMMAND(BakeStreamingTextureTileCommand)(
		[Scene, RuntimeVirtualTextureId, MaterialType, RenderTarget, DestRect, Transform, Bounds, MaxLevel, CustomMaterialData](FRHICommandListImmediate& RHICmdList)
	{
		FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();

		TRefCountPtr<IPooledRenderTarget> PooledRenderTarget = CreateRenderTarget(RenderTarget->GetRenderTargetTexture(), TEXT("RenderTarget"));

		FRDGBuilder GraphBuilder(RHICmdList);
		FScenePrimitiveRenderingContextScopeHelper RenderingScope(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, *Scene));

		RuntimeVirtualTexture::FRenderPageBatchDesc Desc;
		Desc.SceneRenderer = RenderingScope.ScenePrimitiveRenderingContext->GetSceneRenderer();
		Desc.RuntimeVirtualTextureId = RuntimeVirtualTextureId;
		Desc.UVToWorld = Transform;
		Desc.WorldBounds = Bounds;
		Desc.MaterialType = MaterialType;
		Desc.MaxLevel = IntCastChecked<uint8>(MaxLevel);
		Desc.bClearTextures = true;
		Desc.bIsThumbnails = true;
		Desc.FixedColor = FLinearColor::Transparent;
		Desc.CustomMaterialData = CustomMaterialData;
		Desc.NumPageDescs = 1;
		Desc.Targets[0].PooledRenderTarget = PooledRenderTarget;
		Desc.PageDescs[0].DestRect[0] = DestRect;
		Desc.PageDescs[0].UVRange = FBox2D(FVector2D(0, 0), FVector2D(1, 1));
		Desc.PageDescs[0].vLevel = IntCastChecked<uint8>(MaxLevel);

		RuntimeVirtualTexture::RenderPages(GraphBuilder, Desc);

		GraphBuilder.Execute();
	});
}
