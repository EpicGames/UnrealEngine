// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "UObject/UObjectGlobals.h"

#include "RuntimeVirtualTextureThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class UObject;

UCLASS(MinimalAPI)
class URuntimeVirtualTextureThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	//~ Begin UThumbnailRenderer Interface.
	virtual bool CanVisualizeAsset(UObject* Object);
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	//~ EndUThumbnailRenderer Interface.
};
