// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConvertToVirtualTexture.h"

#include "ContentBrowserMenuContexts.h"
#include "Engine/Texture2D.h"
#include "Styling/AppStyle.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialFunctionInstance.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Particles/SubUVAnimation.h"

#include "Editor.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Misc/FeedbackContext.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "IAssetTools.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

#define LOCTEXT_NAMESPACE "SConvertToVirtualTexture"

void SConvertToVirtualTexture::Construct(const FArguments& InArgs, bool bInBackwards)
{
	UserResponse = FConvertToVTDlg::Cancel;
	ParentWindow = InArgs._ParentWindow.Get();
	static FName ErrorIcon = "MessageLog.Error";

	for (int i = 0; i < 16; i++)
	{
		TextureSizes.Add(MakeShareable(new int32(1 << i)));
	}
	const int32 InitiallySelectedIndex = bInBackwards ? 15 : 0;
	ThresholdValue = *TextureSizes[InitiallySelectedIndex];

	this->ChildSlot[
		SNew(SVerticalBox)
		// Textbox at the top giving an introductory message
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Visibility(this, &SConvertToVirtualTexture::GetIntroMessageVisibility)
				.Text(this, &SConvertToVirtualTexture::GetIntroMessage)
			]
		// Error message at the top giving a common error message
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				.Visibility(this, &SConvertToVirtualTexture::GetErrorMessageVisibility)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(ErrorIcon))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(this, &SConvertToVirtualTexture::GetErrorMessage)
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SSeparator)
			]
		// The actual list of assets
		+ SVerticalBox::Slot()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SBorder)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						SAssignNew(AssetListContainer, SVerticalBox)
					]
				]
			]
		// The bottom row of widgets: texture size selector
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ConvertToVT_Size", "Texture size threshold: "))
					.AutoWrapText(true)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SComboBox<TSharedPtr<int32>>)
					.OptionsSource(&TextureSizes)
					.OnSelectionChanged(this, &SConvertToVirtualTexture::OnThresholdChanged)
					.OnGenerateWidget(this, &SConvertToVirtualTexture::OnGenerateThresholdWidget)
					.InitiallySelectedItem(TextureSizes[InitiallySelectedIndex])
					[
						SNew(STextBlock)
						.Text(this, &SConvertToVirtualTexture::GetThresholdText)
					]
				]
			]
		// Separator
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SSeparator)
			]
		// Dialog ok/cancel buttons
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SConvertToVirtualTexture::OnButtonClick, FConvertToVTDlg::Confirm)
					.IsEnabled(this, &SConvertToVirtualTexture::GetOkButtonEnabled)
					.Text(LOCTEXT("ConvertToVT_OK", "OK"))
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SConvertToVirtualTexture::OnButtonClick, FConvertToVTDlg::Cancel)
					.Text(LOCTEXT("ConvertToVT_Cancel", "Cancel"))
				]
			]
	];

	// will be done by SetUserTextures :
	//UpdateList();
}

void SConvertToVirtualTexture::SetBackwards(bool bSetBackwards)
{
	Worker.SetConversionDirection(bSetBackwards);
	bBackwards = bSetBackwards;
}

void SConvertToVirtualTexture::SetUserTextures(const TArray<UTexture2D *> &Textures)
{
	Worker.UserTextures = ObjectPtrWrap(Textures);
	UpdateList();
}

FConvertToVTDlg::EResult SConvertToVirtualTexture::GetUserResponse() const
{
	return UserResponse;
}

TSharedRef<SWidget> SConvertToVirtualTexture::CreateAssetLine(int index, const FAssetData &Asset, const FConversionStatus &Status)
{
	const bool bEngineAsset = Asset.PackagePath.ToString().StartsWith(TEXT("/Engine/"));
	FName SeverityIcon = NAME_None;
	FText DetailedInfoText;

	if (Status.UserSelected)
	{
		if (Status.InvalidMaterialUsage)
		{
			SeverityIcon = "MessageLog.Error";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_InvalidUsage", "The texture could not be converted to VT due to its usage in materials (may be connected to a property that doesn't support VT).");
		}
		else if (Status.NonPowerOf2)
		{
			SeverityIcon = "MessageLog.Error";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_NonPowerOf2", "The texture could not be converted to VT because its size is not a power of 2.");
		}
		else if (bEngineAsset)
		{
			SeverityIcon = "MessageLog.Note";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_Engine", "The texture is an engine asset, a converted copy will be created in the current project.");
		}
		else if (Status.UnderSized)
		{
			SeverityIcon = "MessageLog.Note";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_Undersize", "The texture was under the threshold size but should still be converted to vt because it's using a shared parameter with other VT textures.");
		}
	}
	else
	{
		if (Status.InvalidMaterialUsage)
		{
			SeverityIcon = "MessageLog.Warning";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_InvalidUsageNotSelected", "The texture could not be converted to VT due to its usage in materials (may be connected to a property that doesn't support VT).");
		}
		else if (Status.NonPowerOf2)
		{
			SeverityIcon = "MessageLog.Warning";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_NonPowerOf2NotSelected", "The texture could not be converted to VT because its size is not a power of 2.");
		}
		else if (bEngineAsset)
		{
			SeverityIcon = "MessageLog.Note";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_EngineNotSelected", "The engine asset texture was not selected but a converted copy will be created in the current project, because it's using a shared parameter with other VT textures.");
		}
		else if (Status.UnderSized)
		{
			SeverityIcon = "MessageLog.Note";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_UndersizeNotSelected", "The texture was not selected and furthermore is under the threshold size but should still be converted to vt because it's using a shared parameter with other VT textures.");
		}
		else
		{
			SeverityIcon = "MessageLog.Note";
			DetailedInfoText = LOCTEXT("ConvertToVT_ToolTip_NotSelected", "The texture was not selected but should still be converted to vt because it's using a shared parameter with other VT textures.");
		}
	}

	auto Result =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			// Class icon
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.Padding(2.0f)
				[
					SNew(SImage)
					.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(),
				(Asset.GetClass() == UTexture2D::StaticClass()) ?
						"ClassIcon.Texture2D" :
						((Asset.GetClass() == UMaterialFunction::StaticClass()) ? "ClassIcon.MaterialFunction" : "ClassIcon.Material")).GetIcon())
				]
			]
			// Error/warning icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Padding(2.0f)
				[
					(SeverityIcon == NAME_None) ? SNullWidget::NullWidget :
					static_cast<TSharedRef<SWidget>>(SNew(SImage).Image(FAppStyle::GetBrush(SeverityIcon)))
				]
			]
			// Fold out button with asset name
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
					(DetailedInfoText.IsEmpty())
					? static_cast<TSharedRef<SWidget>>(SNew(STextBlock)
						.Text(FText::FromString(Asset.GetObjectPathString())))
				: SNew(SButton)
				.ButtonStyle(FCoreStyle::Get(), "NoBorder")
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ClickMethod(EButtonClickMethod::MouseDown)
				.OnClicked(this, &SConvertToVirtualTexture::OnExpanderClicked, index)
				.ContentPadding(0.f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				//.Text(FText::FromName(Asset.ObjectPath))
				[
					// Fold out icon
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(this, &SConvertToVirtualTexture::GetExpanderImage, index)
					.ColorAndOpacity(FSlateColor::UseForeground())
					]
					// Fold out text
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(Asset.GetObjectPathString()))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(DetailedInfoText)
			.Visibility(this, &SConvertToVirtualTexture::GetDetailVisibility, index)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			//SNew(STextBlock)
			//.Text(GetAuditTrailText(Asset))
			//.Visibility(this, &SConvertToVirtualTexture::GetDetailVisibility, index)
			GetAuditTrailText(Asset, index)
		];
	return Result;
}

TSharedRef<SWidget> SConvertToVirtualTexture::GetAuditTrailText(const FAssetData &Asset, int32 index)
{
	//FString Result;
	UObject *MaybeOk = Asset.GetAsset();
	auto Trail = Worker.AuditTrail.Find(MaybeOk);

	TSharedRef<SVerticalBox> Box = SNew(SVerticalBox)
	.Visibility(this, &SConvertToVirtualTexture::GetDetailVisibility, index);

	Box->AddSlot()
	.AutoHeight()
	.Padding(8.0f, 4.0f, 8.0f, 4.0f)
	[
		SNew(SSeparator)
	];

	Box->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("ConvertToVT_AuditTrail","This texture was included because of the following dependencies:"))
	];



	while (Trail)
	{
		Box->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(FText::FromString(Trail->PathDescription))
		];

		if (Trail->Destination)
		{
			Box->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Trail->Destination->GetPathName()))
			];

			// May return null at which point we'll break out of the loop
			auto NewTrail = Worker.AuditTrail.Find(Trail->Destination);
			if (NewTrail == nullptr || NewTrail == Trail)
			{
				break;
			}
			Trail = NewTrail;
		}
		else
		{
			break;
		}

		//TODO: We just display the first step only as there isn't much more usefull info
		//in the trail anyway for now and it could actually confuse people seeing half a trail.
		break;
	}

	return Box;
}

void SConvertToVirtualTexture::UpdateList()
{
	Worker.FilterList(ThresholdValue);

	AssetList.Empty();
	AssetStatus.Empty();

	auto CheckUnderSized = [] (const UTexture2D * Texture, int SizeThreshold, bool bConvertBackward)
	{
		// code dupe from FVirtualTextureConversionWorker::FilterList
		
		bool DoInclude;

		// don't use Texture->GetSizeX() as it can be Default texture
		FIntPoint Size = Texture->Source.GetLogicalSize();
		uint64 TexturePixelCount = (uint64) Size.X * Size.Y;

		DoInclude = ( TexturePixelCount >= (uint64)SizeThreshold * SizeThreshold );
			
		// for Backwards this should be the other way around
		//	we want to filter textures *Smaller* than SizeThreshold
		if ( bConvertBackward ) DoInclude = ! DoInclude;

		return ! DoInclude;
	};

	for (UTexture2D *Texture : Worker.Textures)
	{
		AssetList.Add(Texture);
		FConversionStatus* Status = new(AssetStatus) FConversionStatus();
		Status->UserSelected = Worker.UserTextures.Contains(Texture);
		Status->UnderSized = CheckUnderSized(Texture,ThresholdValue,bBackwards);
		Status->NonPowerOf2 = !Texture->Source.AreAllBlocksPowerOfTwo() && Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::None;
	}

	for (UTexture2D* Texture : Worker.MaterialRejectedTextures)
	{
		AssetList.Add(Texture);
		FConversionStatus* Status = new(AssetStatus) FConversionStatus();
		Status->UserSelected = Worker.UserTextures.Contains(Texture);
		Status->UnderSized = CheckUnderSized(Texture,ThresholdValue,bBackwards);
		Status->NonPowerOf2 = !Texture->Source.AreAllBlocksPowerOfTwo() && Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::None;
		Status->InvalidMaterialUsage = true;
	}

	for (UMaterial *RootMat : Worker.Materials)
	{
		// Don't want to show anything from the transient package, this may include preview materials from any currently active material editor
		// We patch these up so material editor remains valid, but not useful to display this to user
		if (RootMat->GetOutermost() != GetTransientPackage())
		{
			AssetList.Add(RootMat);
			AssetStatus.Add(FConversionStatus());
		}
	}

	for (UMaterialFunctionInterface* Func : Worker.Functions)
	{
		if (Func->GetOutermost() != GetTransientPackage())
		{
			AssetList.Add(Func);
			AssetStatus.Add(FConversionStatus());
		}
	}

	check(AssetList.Num() == AssetStatus.Num());
	AssetListContainer->ClearChildren();

	for (int Id = 0; Id < AssetList.Num(); Id++)
	{
		AssetListContainer->AddSlot()
			.AutoHeight()
			[
				CreateAssetLine(Id, AssetList[Id], AssetStatus[Id])
			];
	}

	ErrorMessage = FText();
}


FReply SConvertToVirtualTexture::OnButtonClick(FConvertToVTDlg::EResult ButtonID)
{
	ParentWindow->RequestDestroyWindow();
	UserResponse = ButtonID;

	if (ButtonID == FConvertToVTDlg::EResult::Confirm)
	{
		Worker.DoConvert();
	}

	return FReply::Handled();
}

void SConvertToVirtualTexture::OnThresholdChanged(TSharedPtr<int32> InSelectedItem, ESelectInfo::Type SelectInfo)
{
	ThresholdValue = *InSelectedItem;
	UpdateList();
}

FText SConvertToVirtualTexture::GetThresholdText() const
{
	return FText::FromString(FString::Format(TEXT("{0}"), TArray<FStringFormatArg>({ ThresholdValue })));
}

TSharedRef<SWidget> SConvertToVirtualTexture::OnGenerateThresholdWidget(TSharedPtr<int32> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(FString::Format(TEXT("{0}"), TArray<FStringFormatArg>({ *InItem }))));
}

FReply SConvertToVirtualTexture::OnExpanderClicked(int index)
{
	if (ExpandedIndexes.Contains(index))
	{
		ExpandedIndexes.Remove(index);
	}
	else
	{
		ExpandedIndexes.Add(index);
	}
	return FReply::Handled();
}

bool SConvertToVirtualTexture::GetOkButtonEnabled() const
{
	return ErrorMessage.IsEmpty();
}

EVisibility SConvertToVirtualTexture::GetDetailVisibility(int index) const
{
	return (ExpandedIndexes.Contains(index)) ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SConvertToVirtualTexture::GetExpanderImage(int index) const
{
	FName ResourceName;
	if (GetDetailVisibility(index) == EVisibility::Visible)
	{
		static FName ExpandedName = "TreeArrow_Expanded";
		ResourceName = ExpandedName;
	}
	else
	{
		static FName CollapsedName = "TreeArrow_Collapsed";
		ResourceName = CollapsedName;
	}
	return FCoreStyle::Get().GetBrush(ResourceName);
}

EVisibility SConvertToVirtualTexture::GetIntroMessageVisibility() const
{
	return (IntroMessage.IsEmpty()) ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SConvertToVirtualTexture::GetErrorMessageVisibility() const
{
	return (ErrorMessage.IsEmpty()) ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SConvertToVirtualTexture::GetThresholdVisibility() const
{
	return (bThresholdVisible) ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SConvertToVirtualTexture::GetIntroMessage() const
{
	return IntroMessage;
}

FText SConvertToVirtualTexture::GetErrorMessage() const
{
	return ErrorMessage;
}

FConvertToVTDlg::FConvertToVTDlg(const TArray<UTexture2D *> &Textures, bool bBackwards)
{
	if (FSlateApplication::IsInitialized())
	{
		DialogWindow = SNew(SWindow)
			.Title((bBackwards) ? 
				LOCTEXT("ConvertToVTDlgTitle_Backwards", "Convert VT to Regular if < Threshold") : 
				LOCTEXT("ConvertToVTDlgTitle", "Convert To VT Textures >= Threshold"))
			.SupportsMinimize(false).SupportsMaximize(false)
			.ClientSize(FVector2D(500, 500));

		TSharedPtr<SBorder> DialogWrapper =
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.0f)
			[
				SAssignNew(DialogWidget, SConvertToVirtualTexture, bBackwards)
				.ParentWindow(DialogWindow)
			];

		// SetUserTextures will do UpdateList :
		DialogWidget->SetBackwards(bBackwards);
		DialogWidget->SetUserTextures(Textures);
		DialogWindow->SetContent(DialogWrapper.ToSharedRef());
	}
}

FConvertToVTDlg::EResult FConvertToVTDlg::ShowModal()
{
	//Show Dialog
	GEditor->EditorAddModalWindow(DialogWindow.ToSharedRef());
	EResult UserResponse = (EResult)DialogWidget->GetUserResponse();
	DialogWindow->GetParentWindow()->RemoveDescendantWindow(DialogWindow.ToSharedRef());
	return UserResponse;
}


void SConvertToVirtualTexture::ConvertVTTexture(TArray<UTexture2D*> InTextures, bool backwards)
{
	TArray<UTexture2D*> UserTextures; // The original selection of the user
	
	for (UTexture2D* Texture : InTextures)
	{
		if (Texture != nullptr && Texture->VirtualTextureStreaming == backwards)
		{
			UserTextures.Add(Texture);
		}
	}

	FConvertToVTDlg ConvertDlg(UserTextures, backwards);
	ConvertDlg.ShowModal();
}

#undef LOCTEXT_NAMESPACE
