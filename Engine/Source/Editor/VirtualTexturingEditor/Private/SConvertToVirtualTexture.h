// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/Object.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Styling/AppStyle.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "GameplayTagsManager.h"
#include "VirtualTextureConversionWorker.h"

class UTexture2D;

struct FConversionStatus
{
	bool UserSelected = true; // Originally selected by the user
	bool UnderSized = false; // Too small to convert according to user settings
	bool NonPowerOf2 = false; // Not a power of 2, can't convert
	bool InvalidMaterialUsage = false;
};

/**
* FConvertToVTDlg
*
* Wrapper class for SConvertToVTDlg. This class creates and launches a dialog then awaits the
* result to return to the user.
*/
class FConvertToVTDlg
{
public:
	enum EResult
	{
		Cancel = 0,			// No/Cancel, normal usage would stop the current action
		Confirm = 1,		// Yes/Ok/Etc, normal usage would continue with action
	};

	FConvertToVTDlg(const TArray<UTexture2D*>& Textures, bool bBackwards);

	/**  Shows the dialog box and waits for the user to respond. */
	EResult ShowModal();

private:
	TSharedPtr<SWindow> DialogWindow;
	TSharedPtr<class SConvertToVirtualTexture> DialogWidget;
};

struct FMaterialItem
{
	FName Name;
};

class SConvertToVirtualTexture : public SCompoundWidget
{
public:
	static void ConvertVTTexture(TArray<UTexture2D*> Objects, bool backwards);
	
public:

	SLATE_BEGIN_ARGS(SConvertToVirtualTexture)
	{}
	/** Window in which this widget resides */
	SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, bool bBackwards);

	void SetBackwards(bool bSetBackwards);

	void SetUserTextures(const TArray<UTexture2D *> &Textures);

	FConvertToVTDlg::EResult GetUserResponse() const;

private:

	/**
	* Creates a single line showing an asset and it's status related to VT conversion
	*/
	TSharedRef<SWidget> CreateAssetLine(int index, const FAssetData &Asset, const FConversionStatus &Status);

	TSharedRef<SWidget> GetAuditTrailText(const FAssetData &Asset, int32 index);

	void UpdateList();

	FReply OnButtonClick(FConvertToVTDlg::EResult ButtonID);

	void OnThresholdChanged(TSharedPtr<int32> InSelectedItem, ESelectInfo::Type SelectInfo);

	FText GetThresholdText() const;

	TSharedRef<SWidget> OnGenerateThresholdWidget(TSharedPtr<int32> InItem);

	bool GetFilterButtonEnabled() const;

	FReply OnFilterButtonClicked();
	
	FReply OnExpanderClicked(int index);

	bool GetOkButtonEnabled() const;
	
	EVisibility GetDetailVisibility(int index) const;

	const FSlateBrush* GetExpanderImage(int index) const;

	EVisibility GetIntroMessageVisibility() const;

	EVisibility GetErrorMessageVisibility() const;

	EVisibility GetThresholdVisibility() const;

	FText GetIntroMessage() const;

	FText GetErrorMessage() const;
	
	TArray<int> ExpandedIndexes;

	bool bBackwards;
	FText IntroMessage;
	FText ErrorMessage;
	FConvertToVTDlg::EResult	 UserResponse;

	TSharedPtr<SVerticalBox>	 AssetListContainer;
	//TSharedPtr<STextBlock>		 MessageTextBlock;
	//TSharedPtr<SHorizontalBox>	 ThresholdContainer;
	//TSharedPtr<SHorizontalBox>	 ErrorContainer;
	//TSharedPtr<SButton>			 OkButton;

	/** Pointer to the window which holds this Widget, required for modal control */
	TSharedPtr<SWindow>			 ParentWindow;

	FVirtualTextureConversionWorker Worker;
	TArray<FAssetData> AssetList;
	TArray<FConversionStatus> AssetStatus;

	int ThresholdValue;
	bool bThresholdVisible;

	TArray<TSharedPtr<int32>> TextureSizes;
};
