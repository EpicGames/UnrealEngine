// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureBuilderFactory.h"

#include "AssetTypeCategories.h"
#include "VT/VirtualTextureBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTextureBuilderFactory)

#define LOCTEXT_NAMESPACE "VirtualTextureBuilderFactory"

UVirtualTextureBuilderFactory::UVirtualTextureBuilderFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UVirtualTextureBuilder::StaticClass();
	
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
}

UObject* UVirtualTextureBuilderFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UVirtualTextureBuilder>(InParent, Class, Name, Flags);
}

FText UVirtualTextureBuilderFactory::GetDisplayName() const
{
	return FText(LOCTEXT("DisplayName", "Streaming Runtime Virtual Texture"));
}

FText UVirtualTextureBuilderFactory::GetToolTip() const
{
	return LOCTEXT("ToolTip", "The baked version of a runtime virtual texture (RVT) : this allows to capture the low mips of the RVT in order to save on rendering time, "
		"so that the baked pages are streamed directly from disk instead of being rendered at runtime.\n"
		"In order to work, this asset has to be associated with a Runtime Virtual Texture component. It also gets built through it.");
}

FString UVirtualTextureBuilderFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewStreamingRuntimeVirtualTexture"));
}

#undef LOCTEXT_NAMESPACE