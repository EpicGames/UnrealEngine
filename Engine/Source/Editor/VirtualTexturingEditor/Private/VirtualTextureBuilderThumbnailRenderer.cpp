// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureBuilderThumbnailRenderer.h"
#include "VT/VirtualTexture.h"
#include "VT/VirtualTextureBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTextureBuilderThumbnailRenderer)

UVirtualTextureBuilderThumbnailRenderer::UVirtualTextureBuilderThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UVirtualTextureBuilderThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UVirtualTextureBuilder* VirtualTextureBuilder = Cast<UVirtualTextureBuilder>(Object);
	UVirtualTexture2D* Texture = VirtualTextureBuilder != nullptr ? ToRawPtr(VirtualTextureBuilder->Texture) : nullptr;
	return Texture != nullptr ? UTextureThumbnailRenderer::CanVisualizeAsset(Texture) : false;
}

void UVirtualTextureBuilderThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UVirtualTextureBuilder* VirtualTextureBuilder = Cast<UVirtualTextureBuilder>(Object);
	UVirtualTexture2D* Texture = VirtualTextureBuilder != nullptr ? ToRawPtr(VirtualTextureBuilder->Texture) : nullptr;
	if (Texture != nullptr)
	{
		UTextureThumbnailRenderer::Draw(Texture, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
	}
}
