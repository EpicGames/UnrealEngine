// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureConversionWorker.h"

#include "Engine/Texture2D.h"
#include "Misc/PackageName.h"
#include "Materials/Material.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialShared.h"
#include "MaterialEditor/PreviewMaterial.h"
#include "Factories/MaterialFactoryNew.h"
#include "EditorFramework/AssetImportData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ObjectTools.h"

#include "Editor.h"
#include "Misc/ScopedSlowTask.h"
#include "EditorSupportDelegates.h"
#include "IMaterialEditor.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialEditingLibrary.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "AssetVTConversion"

DEFINE_LOG_CATEGORY_STATIC(LogVirtualTextureConversion, Log, All);

namespace VTConversionWorkerUtil
{
	template <class T> void GetReferencersOfType(UObject *Object, TArray<T*> &OutObjects)
	{
		TArray<FAssetData> OutAssetDatas;
		UAssetRegistryHelpers::FindReferencersOfAssetOfClass(Object, { T::StaticClass() }, OutAssetDatas);

		FScopedSlowTask SlowTask(static_cast<float>(OutAssetDatas.Num()), LOCTEXT("ConvertToVT_Progress_LoadingObjects", "Loading Objects..."));

		for (auto Data : OutAssetDatas)
		{
			SlowTask.EnterProgressFrame();
			if (Data.IsInstanceOf(T::StaticClass()))
			{
				T *IsOk = Cast<T>(Data.GetAsset());
				if (IsOk != nullptr)
				{
					OutObjects.Add(IsOk);
				}
			}
		}
	}
}

static void GetPreviewMaterials(UTexture2D* InTexture, TArray<UMaterialInterface*>& OutMaterials)
{
	for (TObjectIterator<UPreviewMaterial> It; It; ++It)
	{
		UMaterial* Material = *It;
		for (uint32 PropertyIndex = 0u; PropertyIndex < MP_MAX; ++PropertyIndex)
		{
			const EMaterialProperty PropertyToValidate = (EMaterialProperty)PropertyIndex;
			if (Material->IsTextureReferencedByProperty(PropertyToValidate, InTexture))
			{
				OutMaterials.Add(Material);
				break;
			}
		}
	}
}

bool IsVirtualTextureValidForMaterial(UMaterialInterface* InMaterial, UTexture2D* InTexture)
{
	for (uint32 PropertyIndex = 0u; PropertyIndex < MP_MAX; ++PropertyIndex)
	{
		const EMaterialProperty PropertyToValidate = (EMaterialProperty)PropertyIndex;
		const EShaderFrequency ShaderFrequencyToValidate = FMaterialAttributeDefinitionMap::GetShaderFrequency(PropertyToValidate);
		if (PropertyToValidate == MP_OpacityMask || ShaderFrequencyToValidate != SF_Pixel)
		{
			// see if the texture is referenced by a property that doesn't support virtual texture access
			if (InMaterial->IsTextureReferencedByProperty(PropertyToValidate, InTexture))
			{
				return false;
			}
		}
	}

	return true;
}

void FVirtualTextureConversionWorker::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObjects( UserTextures );
	Collector.AddReferencedObjects( Textures );
	Collector.AddReferencedObjects( Materials );
	Collector.AddReferencedObjects( Functions );
	Collector.AddReferencedObjects( SizeRejectedTextures );
	Collector.AddReferencedObjects( MaterialRejectedTextures );

	for (auto& KV : AuditTrail)
	{
		Collector.AddReferencedObject(KV.Key);
	}
}

void FVirtualTextureConversionWorker::FindAllTexturesAndMaterials_Iteration(TSet<UMaterial*>& InAffectedMaterials,
	TSet<UMaterialFunctionInterface*>& InAffectedFunctions,
	TSet<UTexture2D*>& InAffectedTextures,
																																						TSet<TObjectPtr<UTexture2D>>& InInvalidTextures,
	FScopedSlowTask& Task)
{
	TArray<UMaterialInterface *>MaterialInferfaces; // All parents and instances
	TArray<UMaterialInterface *>MaterialHeap; // Temporary work heap
	TArray<UMaterialFunctionInterface *>FunctionInferfaces; // All parents and instances
	TArray<UMaterialFunctionInterface *>FunctionHeap; // Temporary work heap
	TMap<UMaterial*, TSet<FMaterialParameterInfo>> ParametersToVtIze;
	TMap<UMaterialFunctionInterface*, TSet<FMaterialParameterInfo>> FunctionParametersToVtIze;

	// Find all materials that reference the passed in textures
	// This will also load these materials.
	TArray<UMaterialInterface *> MaterialsUsedByAffectedTextures;
	for (TSet<UTexture2D*>::TIterator TextureIter(InAffectedTextures); TextureIter; ++TextureIter)
	{
		UTexture2D* Tex2D = *TextureIter;

		Task.EnterProgressFrame();
		TArray<UMaterialInterface *> MaterialsUsingTexture;
		VTConversionWorkerUtil::GetReferencersOfType(Tex2D, MaterialsUsingTexture);

		/*for (auto Material : MaterialsUsingTexture)
		{
			AuditTrail.Add(Material, FAuditTrail(
				Tex2D,
				FString::Printf(TEXT("references texture"))
			));
		}*/

		// Check all materials using texture, make sure we're able to convert this texture
		bool bIsVirtualTextureValid = true;
		/*
		for (UMaterialInterface* Material : MaterialsUsingTexture)
		{
			if (!bConvertBackward && !IsVirtualTextureValidForMaterial(Material, Tex2D))
			{
				AuditTrail.Add(Tex2D, FAuditTrail(
					Material,
					TEXT("does not support VT usage on material")));
				bIsVirtualTextureValid = false;
				break;
			}
		}
		*/

		if (bIsVirtualTextureValid)
		{
			// If this is an engine texture, we'll make a copy and update any needed references to point to the new copy
			// since we're not changing the original texture, we don't need to bring in any additional dependencies
			// Non-power-2 textures won't convert to VT, don't bring any dependencies for them
			if (!Tex2D->GetPathName().StartsWith("/Engine/") &&
				(Tex2D->Source.AreAllBlocksPowerOfTwo() || Tex2D->PowerOfTwoMode != ETexturePowerOfTwoSetting::None))
			{
				// Also get any preview materials that reference the given texture
				// We need to convert these to ensure any active material editors remain valid
				GetPreviewMaterials(Tex2D, MaterialsUsedByAffectedTextures);

				MaterialsUsedByAffectedTextures.Append(MaterialsUsingTexture);
			}
		}
		else
		{
			TextureIter.RemoveCurrent();
			InInvalidTextures.Add(Tex2D);
		}
	}

	// Find all materials that reference the passed in functions
	// This will also load these materials.
	for (UMaterialFunctionInterface* Func : InAffectedFunctions)
	{
		Task.EnterProgressFrame();
		TArray<UMaterialInterface *> MaterialsUsingFunction;
		VTConversionWorkerUtil::GetReferencersOfType(Func, MaterialsUsingFunction);

		/*for (auto Material : MaterialsUsingFunction)
		{
			AuditTrail.Add(Material, FAuditTrail(
				Func,
				FString::Printf(TEXT("references function"))
			));
		}*/

		MaterialsUsedByAffectedTextures.Append(MaterialsUsingFunction);
	}

	// Find all the root materials of the found instances and add them to our
	// working lists.
	for (UMaterialInterface *If : MaterialsUsedByAffectedTextures)
	{
		// It's a material?
		UMaterial *Material = Cast<UMaterial>(If);

		// It's a material instance? we're only interested in it's root for now
		UMaterialInstance *MaterialInstance = Cast<UMaterialInstance>(If);
		if (MaterialInstance != nullptr)
		{
			Material = MaterialInstance->GetMaterial();
		}

		check(Material); // It's something else entirely??
		MaterialInferfaces.AddUnique(Material);
		MaterialHeap.AddUnique(Material);
		bool bAlreadyExists;
		InAffectedMaterials.Add(Material, &bAlreadyExists);
		if (!bAlreadyExists)
		{
			AuditTrail.Add(Material, FAuditTrail(
				If,
				FString::Printf(TEXT("is the child  of"))
			));
		}
	}

	// We now have a set of "root" materials which will be affected by changing InTextures to VT.
	// Now find all children of these materials which will also be affected trough parameters now requiring VT textures being set on them.
	// This will again load any child instances and their dependencies which aren't loaded yet
	TArray<UMaterialInterface *>VisitedMaterials;

	while (MaterialHeap.Num() > 0)
	{
		UMaterialInterface *ParentMaterial = MaterialHeap[0];
		MaterialHeap.RemoveAt(0);

		if (VisitedMaterials.Contains(ParentMaterial))
		{
			UE_LOGF(LogVirtualTextureConversion, Display, "Circular inheritance chain!? %ls", *ParentMaterial->GetName());
			continue;
		}

		VisitedMaterials.Add(ParentMaterial);

		// Check all parameters of the current material. If they reference a texture
		// we want to convert to VT flag the parameter (this will then cause all textures assigned to this parameter to convert to vt as well)
		TArray<FMaterialParameterInfo> ParameterInfos;
		TArray<FGuid> ParameterGuids;
		ParentMaterial->GetAllTextureParameterInfo(ParameterInfos, ParameterGuids);

		for (FMaterialParameterInfo& ParamInfo : ParameterInfos)
		{
			UTexture *ParamValue = nullptr;
			if (ParentMaterial->GetTextureParameterValue(ParamInfo, ParamValue))
			{
				UTexture2D *ParamValue2D = Cast<UTexture2D>(ParamValue);
				if (ParamValue2D != nullptr)
				{
					if (InAffectedTextures.Contains(ParamValue2D))
					{
						//UE_LOGF(LogVirtualTextureConversion, Display, "Adding parameter %ls because it references  %ls on %ls >> %ls", *ParamInfo.Name.ToString(), *ParamValue2D->GetPathName(), *ParentMaterial->GetMaterial()->GetPathName(), *ParentMaterial->GetPathName());
						ParametersToVtIze.FindOrAdd(ParentMaterial->GetMaterial()).Add(ParamInfo);
					}
				}
			}
		}

		// Find all direct children of this material (children of children will be discovered later by pushing these children on the MaterialHeap).
		TArray<UMaterialInstance*> ParentMaterialInstances;
		Task.EnterProgressFrame();
		VTConversionWorkerUtil::GetReferencersOfType(ParentMaterial, ParentMaterialInstances);

		for (auto MaterialInstance : ParentMaterialInstances)
		{
			//MaterialInstances.AddUnique(MaterialInstance);
			MaterialInferfaces.AddUnique(MaterialInstance);
			//InstancesForMaterial.FindOrAdd(MaterialInstance->GetMaterial()).AddUnique(MaterialInstance);

			// Push on the heap to check materials referencing us recursively
			MaterialHeap.AddUnique(MaterialInstance);
		}
	}
	// We know have a set of root materials and a set of properties to convert to VT
	// Find all textures referenced by these properties
	// These new textures could in turn be referenced by other materials (not in the inheritance chain)
	// which is why we have to run this discovery process iteratively until we don't discover any new
	// materials or textures.

	for (UMaterialInterface *If : MaterialInferfaces)
	{
		UMaterial *Mat = If->GetMaterial();
		for (const FMaterialParameterInfo &Parameter : ParametersToVtIze.FindOrAdd(Mat))
		{
			UTexture *Tex = nullptr;
			if (If->GetTextureParameterValue(Parameter, Tex))
			{
				UTexture2D *Tex2D = Cast<UTexture2D>(Tex);
				if (Tex2D && !Tex2D->VirtualTextureStreaming)
				{
					bool bAlreadyExists;
					InAffectedTextures.Add(Tex2D, &bAlreadyExists);
					if (!bAlreadyExists)
					{
						AuditTrail.Add(Tex2D, FAuditTrail(
							Mat,
							FString::Printf(TEXT("set on parameter %s in instance %s of material"), *Parameter.Name.ToString(), *If->GetName())
						));
					}
				}
			}
		}
	}

	//
	// Pretty much the same again but now for material functions....
	//

	// Find all materials functions that directly reference the passed in textures
	TArray<UMaterialFunctionInterface *> FunctionsUsedByAffectedTextures;
	for (UTexture2D* Tex2D : InAffectedTextures)
	{
		Task.EnterProgressFrame();
		if (!Tex2D->GetPathName().StartsWith("/Engine/"))
		{
			VTConversionWorkerUtil::GetReferencersOfType(Tex2D, FunctionsUsedByAffectedTextures);
		}
	}

	// Find all the root function of the found instances and add them to our
	// working lists.
	for (UMaterialFunctionInterface *If : FunctionsUsedByAffectedTextures)
	{
		// It's a material?
		UMaterialFunction* Function = If->GetBaseFunction();
		if (Function)
		{
			FunctionInferfaces.AddUnique(Function);
			FunctionHeap.AddUnique(Function);
			bool bAlreadyExists;
			InAffectedFunctions.Add(Function, &bAlreadyExists);
			if (!bAlreadyExists)
			{
				AuditTrail.Add(Function, FAuditTrail(
					If,
					FString::Printf(TEXT("this is the parent of"))
				));
			}
		}
	}


	// We have a second class of functions here. That is functions that don't directly reference any textures of interest
	// but where there are material instances that overrides properties in the textures that do reference textures of interest
	for (TPair<UMaterial*, TSet<FMaterialParameterInfo>>& MaterialParametersPair : ParametersToVtIze)
	{
		for (FMaterialParameterInfo& Parameter : MaterialParametersPair.Value)
		{
			TArray<UMaterialFunctionInterface*> DependentFunctions;
			MaterialParametersPair.Key->GetDependentFunctions(DependentFunctions);
			for (UMaterialFunctionInterface* Function : DependentFunctions)
			{
				for (UMaterialExpression* FunctionExpression : Function->GetExpressions())
				{
					if (const UMaterialExpressionTextureSampleParameter* TexParameter = Cast<const UMaterialExpressionTextureSampleParameter>(FunctionExpression))
					{
						if (TexParameter->ParameterName == Parameter.Name)
						{
							FunctionParametersToVtIze.FindOrAdd(Function->GetBaseFunction()).Add(Parameter);
							InAffectedFunctions.Add(Function->GetBaseFunction());
							FunctionHeap.AddUnique(Function->GetBaseFunction());
						}
					}
				}
			}
		}
	}

	// We now have a set of "root" functions which will be affected by changing InTextures to VT.
	// Now find all children of these materials which will also be affected trough parameters now requiring VT textures being set on them.
	// This will again load any child instances and their dependencies which aren't loaded yet
	while (FunctionHeap.Num() > 0)
	{
		UMaterialFunctionInterface *ParentFunction = FunctionHeap[0];
		FunctionHeap.RemoveAt(0);
		{
			UMaterialFunctionInstance* FunctionInstance = Cast<UMaterialFunctionInstance>(ParentFunction);
			if (FunctionInstance)
			{
				// Check all parameters of the current material. If they reference a texture
				// we want to convert to VT flag the parameter (this will then cause all textures assigned to this parameter to convert to vt as well)
				for (const FTextureParameterValue& TextureParameter : FunctionInstance->TextureParameterValues)
				{
					UTexture2D* ParamValue2D = Cast<UTexture2D>(TextureParameter.ParameterValue);
					if (InAffectedTextures.Contains(ParamValue2D))
					{
						FunctionParametersToVtIze.FindOrAdd(ParentFunction->GetBaseFunction()).Add(TextureParameter.ParameterInfo);
					}
				}
			}
		}

		// Find all direct children of this function (children of children will be discovered later by pushing these children on the FunctionHeap).
		TArray<UMaterialFunctionInstance*> ParentFunctionInstances;
		Task.EnterProgressFrame();
		VTConversionWorkerUtil::GetReferencersOfType(ParentFunction, ParentFunctionInstances);

		for (auto FunctionInstance : ParentFunctionInstances)
		{
			FunctionInferfaces.AddUnique(FunctionInstance);

			// Push on the heap to check materials referencing us recursively
			FunctionHeap.AddUnique(FunctionInstance);
		}
	}

	// We know have a set of root functions and a set of properties to convert to VT
	// Find all textures referenced by these properties
	// These new textures could in turn be referenced by other materials and or functions (not in the inheritance chain)
	// which is why we have to run this discovery process iteratively until we don't discover any new
	// materials or textures.

	for (UMaterialFunctionInterface *If : FunctionInferfaces)
	{
		UMaterialFunctionInterface *Func = If->GetBaseFunction();
		for (const FMaterialParameterInfo &Parameter : FunctionParametersToVtIze.FindOrAdd(Func))
		{
			UTexture *Tex = nullptr;
			If->OverrideNamedTextureParameter(Parameter, Tex);
			UTexture2D *Tex2D = Cast<UTexture2D>(Tex);
			if (Tex2D && !Tex2D->VirtualTextureStreaming)
			{
				bool bAlreadyExists;
				InAffectedTextures.Add(Tex2D, &bAlreadyExists);
				if (!bAlreadyExists)
				{
					AuditTrail.Add(Tex2D, FAuditTrail(
						Func,
						FString::Printf(TEXT("set on parameter %s in instance %s"), *Parameter.Name.ToString(), *If->GetPathName())
					));
				}
			}
		}
	}
}

void FVirtualTextureConversionWorker::FindAllTexturesAndMaterials(TArray<TObjectPtr<UMaterial >> &OutAffectedMaterials, TArray<TObjectPtr<UMaterialFunctionInterface >> &OutAffectedFunctions, TArray<TObjectPtr<UTexture2D >> &OutAffectedTextures)
{
	FScopedSlowTask SlowTask(1000.0f, LOCTEXT("ConvertToVT_Progress_FindAllTexturesAndMaterials", "Finding Textures and Materials..."));
	TSet<UMaterial*> AffectedMaterials;
	TSet<UMaterialFunctionInterface*> AffectedFunctions;
	TSet<UTexture2D*> AffectedTextures;
	AffectedMaterials.Append(ObjectPtrDecay(OutAffectedMaterials));
	AffectedFunctions.Append(ObjectPtrDecay(OutAffectedFunctions));
	AffectedTextures.Append(ObjectPtrDecay(OutAffectedTextures));
	int LastNumMaterials = AffectedMaterials.Num();
	int LastNumTextures = AffectedTextures.Num();
	int LastNumFunctions = AffectedFunctions.Num();

	while (true)
	{
		FindAllTexturesAndMaterials_Iteration(AffectedMaterials, AffectedFunctions, AffectedTextures, MaterialRejectedTextures, SlowTask);
		if (AffectedMaterials.Num() == LastNumMaterials && AffectedTextures.Num() == LastNumTextures && AffectedFunctions.Num() == LastNumFunctions)
		{
			break;
		}

		LastNumMaterials = AffectedMaterials.Num();
		LastNumTextures = AffectedTextures.Num();
		LastNumFunctions = AffectedFunctions.Num();
	}
	OutAffectedMaterials = ObjectPtrWrap(AffectedMaterials.Array());
	OutAffectedFunctions = ObjectPtrWrap(AffectedFunctions.Array());
	OutAffectedTextures = ObjectPtrWrap(AffectedTextures.Array());
}

void FVirtualTextureConversionWorker::FilterList(int32 SizeThreshold)
{
	FScopedSlowTask SlowTask(1.0f, LOCTEXT("ConvertToVT_Progress_FindingTextures", "Finding textures to convert..."));
	SlowTask.MakeDialog();

	Textures.Empty();
	Materials.Empty();
	Functions.Empty();
	SizeRejectedTextures.Empty();

	for (UTexture2D *Texture : UserTextures)
	{
		bool DoInclude;

		if ( !Texture->Source.AreAllBlocksPowerOfTwo() && Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::None )
		{
			// not pow2, reject
			DoInclude = false;
		}
		else 
		{
			// don't use Texture->GetSizeX() as it can be Default texture
			FIntPoint Size = Texture->Source.GetLogicalSize();
			uint64 TexturePixelCount = (uint64) Size.X * Size.Y;

			DoInclude = ( TexturePixelCount >= (uint64)SizeThreshold * SizeThreshold );
			
			// for Backwards this should be the other way around
			//	we want to filter textures *Smaller* than SizeThreshold
			if ( bConvertBackward ) DoInclude = ! DoInclude;
		}

		if ( DoInclude )
		{
			Textures.Add(Texture);
		}
		else
		{
			SizeRejectedTextures.Add(Texture);
		}
	}

	SlowTask.EnterProgressFrame();
	FindAllTexturesAndMaterials(Materials, Functions, Textures);
}

void FVirtualTextureConversionWorker::DoConvert()
{
	const bool bVirtualTextureEnable = !bConvertBackward;

	FScopedSlowTask SlowTask(2.0f, LOCTEXT("ConvertToVT_Progress_ConvertingTexturesAndMaterials", "Converting textures and materials..."));
	SlowTask.MakeDialog();

	TMap<UTexture2D*, UTexture2D*> EngineTextureToCopyMap;

	UE_LOGF(LogVirtualTextureConversion, Display, "Beginning conversion...");
	SlowTask.EnterProgressFrame();
	{
		FScopedSlowTask TextureTask(static_cast<float>(Textures.Num()), LOCTEXT("ConvertToVT_Progress_TextureTask", "Updating textures..."));

		for (UTexture2D *Tex : Textures)
		{
			UTexture2D* TextureToUpdate = Tex;
			if (TextureToUpdate->GetPathName().StartsWith(TEXT("/Engine/")) && TextureToUpdate->GetPackage() != GetTransientPackage())
			{
				// rather than modify engine content, create a copy and update that
				// any materials that we modify will be updated to point to the copy
				ObjectTools::FPackageGroupName PGN;
				PGN.GroupName = TEXT("");
				PGN.ObjectName = Tex->GetName().Append(TEXT("_VT"));
				PGN.PackageName = TEXT("/Game/Textures/");
				PGN.PackageName.Append(PGN.ObjectName);

				UPackage* ExistingPackage = FindPackage(NULL, *PGN.PackageName);
				UTexture2D* DuplicateTexture = nullptr;
				if (ExistingPackage)
				{
					UObject* DuplicateObject = StaticFindObject(UTexture2D::StaticClass(), ExistingPackage, *PGN.ObjectName);
					if (DuplicateObject)
					{
						DuplicateTexture = CastChecked<UTexture2D>(DuplicateObject);
					}
				}

				if (!DuplicateTexture || DuplicateTexture->VirtualTextureStreaming != bVirtualTextureEnable)
				{
					// TODO - overwrite previous texture (if it exists), or should we always generate a unique name?
					TSet<UPackage*> ObjectsUserRefusedToFullyLoad;
					UObject* DuplicateObject = ObjectTools::DuplicateSingleObject(Tex, PGN, ObjectsUserRefusedToFullyLoad, false);
					if (DuplicateObject)
					{
						DuplicateTexture = CastChecked<UTexture2D>(DuplicateObject);
					}
				}

				TextureToUpdate = DuplicateTexture;
				EngineTextureToCopyMap.Add(Tex, TextureToUpdate);
				if (!TextureToUpdate)
				{
					UE_LOGF(LogVirtualTextureConversion, Warning, "Failed to duplicate engine texture %ls", *Tex->GetPathName());
				}
			}

			if (TextureToUpdate)
			{
				UE_LOGF(LogVirtualTextureConversion, Display, "Texture %ls", *TextureToUpdate->GetName());
				TextureTask.EnterProgressFrame();

				if (TextureToUpdate->VirtualTextureStreaming != bVirtualTextureEnable)
				{
					TextureToUpdate->PreEditChange(nullptr);
					TextureToUpdate->VirtualTextureStreaming = bVirtualTextureEnable;
					TextureToUpdate->PostEditChange();
				}
			}
		}
	}

	SlowTask.EnterProgressFrame();
	{
		FScopedSlowTask MaterialTask(static_cast<float>((Materials.Num() + Functions.Num()) * 2), LOCTEXT("ConvertToVT_Progress_MaterialTask", "Updating materials..."));

		TArray<UMaterial*> ModifiedMaterials;
		ModifiedMaterials.Reserve(Materials.Num());
		TArray<UMaterialFunctionInterface*> ModifiedMaterialFunctions;
		ModifiedMaterialFunctions.Reserve(Functions.Num());

		for (UMaterial *Mat : Materials)
		{
			UE_LOGF(LogVirtualTextureConversion, Display, "Material %ls", *Mat->GetName());

			MaterialTask.EnterProgressFrame();

			bool MatModified = false;
			for (UMaterialExpression *Expr : Mat->GetExpressions())
			{
				UMaterialExpressionTextureBase *TexExpr = Cast<UMaterialExpressionTextureBase>(Expr);
				if (TexExpr)
				{
					if (Textures.Contains(TexExpr->Texture))
					{
						UE_LOGF(LogVirtualTextureConversion, Display, "Adjusting sampler %ls.", *TexExpr->GetName());
						UTexture2D** FoundTextureCopy = EngineTextureToCopyMap.Find(CastChecked<UTexture2D>(TexExpr->Texture));
						UTexture2D* TextureCopy = nullptr;
						bool bShouldUpdateMaterial = true;
						if (FoundTextureCopy)
						{
							TextureCopy = *FoundTextureCopy;
							if (TextureCopy)
							{
								TexExpr->Texture = TextureCopy;
							}
							else
							{
								// nullptr was set in EngineTextureToCopyMap....this means we failed to create copy of engine texture for this resource
								// bail on updating the material in this case
								bShouldUpdateMaterial = false;
							}
						}

						auto OldType = TexExpr->SamplerType;
						if (bShouldUpdateMaterial)
						{
							TexExpr->AutoSetSampleType();
						}
						if (TextureCopy != nullptr || OldType != TexExpr->SamplerType)
						{
							TexExpr->Modify();
							MatModified = true;
						}
					}
				}
			}

			if (MatModified)
			{
				UE_LOGF(LogVirtualTextureConversion, Display, "Material %ls added to update list.", *Mat->GetName());
				ModifiedMaterials.Add(Mat);
			}
			else
			{
				UE_LOGF(LogVirtualTextureConversion, Display, "Material %ls was not modified, skipping.", *Mat->GetName());
			}
		}

		UMaterialEditingLibrary::RecompileMaterials(ModifiedMaterials, UMaterialEditingLibrary::FOnItemComplete::CreateLambda([&MaterialTask]() { MaterialTask.EnterProgressFrame(); }));

		for (UMaterialFunctionInterface *Func : Functions)
		{
			UE_LOGF(LogVirtualTextureConversion, Display, "Function %ls", *Func->GetName());

			MaterialTask.EnterProgressFrame();

			bool FuncModified = false;
			for (const TObjectPtr<UMaterialExpression>& Expr : Func->GetExpressions())
			{
				UMaterialExpressionTextureBase *TexExpr = Cast<UMaterialExpressionTextureBase>(Expr);
				if (TexExpr)
				{
					if (Textures.Contains(TexExpr->Texture))
					{
						UE_LOGF(LogVirtualTextureConversion, Display, "Adjusting sampler %ls.", *TexExpr->GetName());
						UTexture2D** FoundTextureCopy = EngineTextureToCopyMap.Find(CastChecked<UTexture2D>(TexExpr->Texture));
						UTexture2D* TextureCopy = nullptr;
						bool bShouldUpdateMaterial = true;
						if (FoundTextureCopy)
						{
							TextureCopy = *FoundTextureCopy;
							if (TextureCopy)
							{
								TexExpr->Texture = TextureCopy;
							}
							else
							{
								// nullptr was set in EngineTextureToCopyMap....this means we failed to create copy of engine texture for this resource
								// bail on updating the material in this case
								bShouldUpdateMaterial = false;
							}
						}

						auto OldType = TexExpr->SamplerType;
						if (bShouldUpdateMaterial)
						{
							TexExpr->AutoSetSampleType();
						}
						if (TextureCopy != nullptr || TexExpr->SamplerType != OldType)
						{
							TexExpr->Modify();
							FuncModified = true;
						}
					}
				}
			}

			if (FuncModified)
			{
				UE_LOGF(LogVirtualTextureConversion, Display, "Function %ls added to update list.", *Func->GetName());
				ModifiedMaterialFunctions.Add(Func);
			}
			else
			{
				UE_LOGF(LogVirtualTextureConversion, Display, "Function %ls was not modified, skipping.", *Func->GetName());
			}
		}

		UMaterialEditingLibrary::UpdateMaterialFunctions(ModifiedMaterialFunctions, UMaterialEditingLibrary::FOnItemComplete::CreateLambda([&MaterialTask]() { MaterialTask.EnterProgressFrame(); }));

		// Update expression previews for any open materials.
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		for (UMaterial* Mat : ModifiedMaterials)
		{
			if (IAssetEditorInstance* FoundAssetEditor = AssetEditorSubsystem->FindEditorForAsset(Mat, false))
			{
				IMaterialEditor* MaterialEditor = static_cast<IMaterialEditor*>(FoundAssetEditor);
				MaterialEditor->ForceRefreshExpressionPreviews();
			}
		}

		// Update the world's viewports.
		UE_LOGF(LogVirtualTextureConversion, Display, "Broadcasting to editor.");
		FEditorDelegates::RefreshEditor.Broadcast();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}
}

#undef LOCTEXT_NAMESPACE
