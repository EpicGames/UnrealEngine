// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/GCObject.h"
#include "AssetRegistry/AssetIdentifier.h"

class UTexture2D;
class UMaterial;
class UMaterialFunctionInterface;
struct FScopedSlowTask;
class UClass;
struct FAssetData;

class FVirtualTextureConversionWorker : public FGCObject
{
public:
	FVirtualTextureConversionWorker(bool bInConvertBackward = false) : bConvertBackward(bInConvertBackward){}

	bool bConvertBackward;

	// Fill this array intially with some textures you want to see converted to vt
	TArray<TObjectPtr<UTexture2D>> UserTextures;

	// These will be filled by calling FilterList
	TArray<TObjectPtr<UTexture2D>> Textures;
	TArray<TObjectPtr<UMaterial >>Materials;
	TArray<TObjectPtr<UMaterialFunctionInterface >> Functions;
	TArray<TObjectPtr<UTexture2D>> SizeRejectedTextures;// The original selection of the user filtered by textures that match the threshold size
	TSet<TObjectPtr<UTexture2D>> MaterialRejectedTextures; // Textures that could not be converted due to their usage in materials (connected to unsupported property)

	struct FAuditTrail
	{
		FAuditTrail(UObject *SetObject, const FString &SetString) : Destination(SetObject), PathDescription(SetString) {}
		UObject *Destination; //Next object in the trail
		FString PathDescription; //Description on how we go there
	};

	TMap<TObjectPtr<UObject>, FAuditTrail> AuditTrail;

	void SetConversionDirection(bool bInConvertBackward) { bConvertBackward = bInConvertBackward; }

	void FilterList(int32 SizeThreshold);

	// Based on the filtered lists do the actual VT conversion
	void DoConvert();

	void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FVTConversionWorker");
	}

private:
	void FindAllTexturesAndMaterials_Iteration(TSet<UMaterial*>& InAffectedMaterials,
		TSet<UMaterialFunctionInterface*>& InAffectedFunctions,
		TSet<UTexture2D*>& InAffectedTextures,
		TSet<TObjectPtr<UTexture2D>>& InInvalidTextures,
		FScopedSlowTask& Task);

	void FindAllTexturesAndMaterials(TArray<TObjectPtr<UMaterial >> &OutAffectedMaterials, TArray<TObjectPtr<UMaterialFunctionInterface >> &OutAffectedFunctions, TArray<TObjectPtr<UTexture2D >> &OutAffectedTextures);

};
