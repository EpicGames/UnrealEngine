// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTexturingEditorModule.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "IPlacementModeModule.h"
#include "MeshPaintVirtualTextureThumbnailRenderer.h"
#include "PropertyEditorModule.h"
#include "RuntimeVirtualTextureBuildStreamingMips.h"
#include "RuntimeVirtualTextureDetailsCustomization.h"
#include "RuntimeVirtualTextureThumbnailRenderer.h"
#include "SConvertToVirtualTexture.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/UObjectIterator.h"
#include "VirtualTextureBuilderThumbnailRenderer.h"
#include "VT/MeshPaintVirtualTexture.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureVolume.h"
#include "VT/VirtualTextureBuilder.h"
#include "VirtualTextureConversionWorker.h"

#define LOCTEXT_NAMESPACE "VirtualTexturingEditorModule"

/** Concrete implementation of the IVirtualTexturingEditorModule interface. */
class FVirtualTexturingEditorModule : public IVirtualTexturingEditorModule
{
public:
	//~ Begin IModuleInterface Interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
	//~ End IModuleInterface Interface.

	//~ Begin IVirtualTexturingEditorModule Interface.
	UE_DEPRECATED(5.6, "Use the version that takes a EShadingPath (previously was : EShadingPath::Deferred)")
	virtual bool HasStreamedMips(URuntimeVirtualTextureComponent* InComponent) const override;
	virtual bool HasStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent) const override;
	UE_DEPRECATED(5.6, "Use the version that takes a EShadingPath (previously was : EShadingPath::Deferred)")
	virtual bool BuildStreamedMips(URuntimeVirtualTextureComponent* InComponent) const override;
	virtual bool BuildStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent) const override;
	virtual FBuildAllStreamedMipsResult BuildAllStreamedMips(const FBuildAllStreamedMipsParams& InParams) const override;
	virtual void ConvertVirtualTextures(const TArray<UTexture2D *>& Textures, bool bConvertBackToNonVirtual, const TArray<UMaterial *>* RelatedMaterials /* = nullptr */) const override;
	virtual void ConvertVirtualTexturesWithDialog(const TArray<UTexture2D *>& Textures, bool bConvertBackToNonVirtual) const override;
	virtual TArray<URuntimeVirtualTextureComponent*> GatherRuntimeVirtualTextureComponents(UWorld* InWorld) const override;
	//~ End IVirtualTexturingEditorModule Interface.

private:
	void OnPlacementModeRefresh(FName CategoryName);
};

IMPLEMENT_MODULE(FVirtualTexturingEditorModule, VirtualTexturingEditor);

void FVirtualTexturingEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("RuntimeVirtualTexture", FOnGetDetailCustomizationInstance::CreateStatic(&FRuntimeVirtualTextureDetailsCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("RuntimeVirtualTextureComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FRuntimeVirtualTextureComponentDetailsCustomization::MakeInstance));

	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	PlacementModeModule.OnPlacementModeCategoryRefreshed().AddRaw(this, &FVirtualTexturingEditorModule::OnPlacementModeRefresh);

	UThumbnailManager::Get().RegisterCustomRenderer(URuntimeVirtualTexture::StaticClass(), URuntimeVirtualTextureThumbnailRenderer::StaticClass());
	UThumbnailManager::Get().RegisterCustomRenderer(UVirtualTextureBuilder::StaticClass(), UVirtualTextureBuilderThumbnailRenderer::StaticClass());
	UThumbnailManager::Get().RegisterCustomRenderer(UMeshPaintVirtualTexture::StaticClass(), UMeshPaintVirtualTextureThumbnailRenderer::StaticClass());
}

void FVirtualTexturingEditorModule::ShutdownModule()
{
	if (IPlacementModeModule::IsAvailable())
	{
		IPlacementModeModule::Get().OnPlacementModeCategoryRefreshed().RemoveAll(this);
	}
}

bool FVirtualTexturingEditorModule::SupportsDynamicReloading()
{
	return false;
}

void FVirtualTexturingEditorModule::OnPlacementModeRefresh(FName CategoryName)
{
	static FName VolumeName = FName(TEXT("Volumes"));
	if (CategoryName == VolumeName)
	{
		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
		PlacementModeModule.RegisterPlaceableItem(CategoryName, MakeShareable(new FPlaceableItem(nullptr, FAssetData(ARuntimeVirtualTextureVolume::StaticClass()))));
	}
}

// Deprecated
bool FVirtualTexturingEditorModule::HasStreamedMips(URuntimeVirtualTextureComponent* InComponent) const
{
	return RuntimeVirtualTexture::HasStreamedMips(EShadingPath::Deferred, InComponent);
}

bool FVirtualTexturingEditorModule::HasStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent) const
{
	return RuntimeVirtualTexture::HasStreamedMips(ShadingPath, InComponent);
}

// Deprecated
bool FVirtualTexturingEditorModule::BuildStreamedMips(URuntimeVirtualTextureComponent* InComponent) const
{
	return RuntimeVirtualTexture::BuildStreamedMips(EShadingPath::Deferred, InComponent);
}

bool FVirtualTexturingEditorModule::BuildStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent) const
{
	return RuntimeVirtualTexture::BuildStreamedMips(ShadingPath, InComponent);
}

FBuildAllStreamedMipsResult FVirtualTexturingEditorModule::BuildAllStreamedMips(const FBuildAllStreamedMipsParams& InParams) const
{
	return RuntimeVirtualTexture::BuildAllStreamedMips(InParams);
}

void FVirtualTexturingEditorModule::ConvertVirtualTextures(const TArray<UTexture2D *>& Textures, bool bConvertBackToNonVirtual, const TArray<UMaterial *>* RelatedMaterials /* = nullptr */) const
{
	FVirtualTextureConversionWorker VirtualTextureConversionWorker(bConvertBackToNonVirtual);
	VirtualTextureConversionWorker.UserTextures = ObjectPtrWrap(Textures);
	// FilterList takes a threshold that is either greater or less than according to which direction we are converting in.
	const int32 FilterListSizeThreshold = bConvertBackToNonVirtual ? MAX_int32 : 0;
	VirtualTextureConversionWorker.FilterList(FilterListSizeThreshold);
	if (RelatedMaterials)
	{
		VirtualTextureConversionWorker.Materials.Append(*RelatedMaterials);
	}

	VirtualTextureConversionWorker.DoConvert();
}

void FVirtualTexturingEditorModule::ConvertVirtualTexturesWithDialog(const TArray<UTexture2D*>& Textures, bool bConvertBackToNonVirtual) const
{
	SConvertToVirtualTexture::ConvertVTTexture(Textures, bConvertBackToNonVirtual);
}

TArray<URuntimeVirtualTextureComponent*> FVirtualTexturingEditorModule::GatherRuntimeVirtualTextureComponents(UWorld* InWorld) const
{
	TArray<URuntimeVirtualTextureComponent*> Components;
	for (TObjectIterator<URuntimeVirtualTextureComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->IsRegistered() && (It->GetWorld() == InWorld))
		{
			Components.Add(*It);
		}
	}
	return Components;
}

#undef LOCTEXT_NAMESPACE
