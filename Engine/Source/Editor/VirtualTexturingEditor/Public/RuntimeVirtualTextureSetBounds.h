// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class URuntimeVirtualTextureComponent;

namespace RuntimeVirtualTexture
{
	/** Set the transform on a URuntimeVirtualTextureComponent so that it includes the bounds of all associated primitives in the current world. */
	void VIRTUALTEXTURINGEDITOR_API SetBounds(URuntimeVirtualTextureComponent* InComponent);
};
