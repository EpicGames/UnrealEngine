// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class UTexture2D;
class UMaterial;
class URuntimeVirtualTextureComponent;
enum class EShadingPath;

struct FBuildAllStreamedMipsResult
{
	bool bSuccess = true;
	TSet<UPackage*> ModifiedPackages;
};

struct FBuildAllStreamedMipsParams
{
	/** World for which to build the streamed mips */
	UWorld* World = nullptr;
	/** RVT components for which to build the streamed mips. The components should belong to the World and be properly registered */
	TConstArrayView<URuntimeVirtualTextureComponent*> Components;
	/** If the feature level has been changed because we needed to build the SVT for multiple feature levels, defines whether it should be restored afterwards */
	bool bRestoreFeatureLevelAfterBuilding = true;
};

/** Module for virtual texturing editor extensions. */
class IVirtualTexturingEditorModule : public IModuleInterface
{
public:
	UE_DEPRECATED(5.6, "Use the version that takes a EShadingPath (previously was : EShadingPath::Deferred)")
	virtual bool HasStreamedMips(URuntimeVirtualTextureComponent* InComponent) const PURE_VIRTUAL(IVirtualTexturingEditorModule::HasStreamedMips, return false;);
	
	/**
	 * Returns true if the component describes a runtime virtual texture that has streaming low mips for the specified shading path
	 */
	virtual bool HasStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent) const PURE_VIRTUAL(IVirtualTexturingEditorModule::HasStreamedMips, return false;);
	
	UE_DEPRECATED(5.6, "Use the version that takes a EShadingPath (previously was : EShadingPath::Deferred)")
	virtual bool BuildStreamedMips(URuntimeVirtualTextureComponent* InComponent) const PURE_VIRTUAL(IVirtualTexturingEditorModule::BuildStreamedMips, return false;);
	
	/**
	 * Build the contents of the streaming low mips for a single component and a single shading path.
	 * @param ShadingPath shading path for which to build the SVT (note: EShadingPath::Mobile only if enabled)
	 * @param InComponent RVT component to build the SVT for. It's expected that this component is registered
	 * 
	 * @return true in case of success 
	 */
	virtual bool BuildStreamedMips(EShadingPath ShadingPath, URuntimeVirtualTextureComponent* InComponent) const PURE_VIRTUAL(IVirtualTexturingEditorModule::BuildStreamedMips, return false;);

	/**
	 * Build the contents of the streaming low mips for several components for all supported shading paths (e.g. deferred and mobile, if enabled)
	 * This is the preferred method for ensuring the SVT is up-to-date across all platforms but is slower because of the potential multiple shading paths
	 * @param InWorld world to whom belong the RVT components to bake
	 * @param InComponents components to build the SVT for. It's expected that these components are registered
	 * 
	 * @return a struct containing the details about the whole operation
	 */
	virtual FBuildAllStreamedMipsResult BuildAllStreamedMips(const FBuildAllStreamedMipsParams& InParams) const PURE_VIRTUAL(IVirtualTexturingEditorModule::BuildAllStreamedMips, return {};);
	
	virtual void ConvertVirtualTextures(const TArray<UTexture2D *>& Textures, bool bConvertBackToNonVirtual, const TArray<UMaterial *>* RelatedMaterials /* = nullptr */) const PURE_VIRTUAL(IVirtualTexturingEditorModule::ConvertVirtualTextures, );
	virtual void ConvertVirtualTexturesWithDialog(const TArray<UTexture2D*>& Textures, bool bConvertBackToNonVirtual) const PURE_VIRTUAL(IVirtualTexturingEditorModule::ConvertVirtualTexturesWithDialog, );

	/** 
	 * @return the list of valid RVT components for this world
	 */
	virtual TArray<URuntimeVirtualTextureComponent*> GatherRuntimeVirtualTextureComponents(UWorld* InWorld) const PURE_VIRTUAL(IVirtualTexturingEditorModule::GatherRuntimeVirtualTextureComponents, return {};);
};
