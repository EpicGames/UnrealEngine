// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class VirtualTexturingEditor : ModuleRules
{
	public VirtualTexturingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"AppFramework",
				"AssetRegistry",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"EditorFramework",
				"EditorWidgets",
				"Engine",
				"InputCore",
				"Landscape",
				"MaterialEditor",
				"PlacementMode",
				"PropertyEditor",
				"RenderCore",
				"Renderer",
				"RHI",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}